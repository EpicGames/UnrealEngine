{
	"FileVersion": 3,
	"Version": 0.1,
	"VersionName": "0.1",
	"FriendlyName": "NNERuntimeBasicCpu",
	"Description": "Performant, cross-platform, CPU runtime for the NNE plugin that supports basic models.",
	"Category": "Experimental",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "NNERuntimeBasicCpu",
			"Type": "RuntimeAndProgram",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": []
}