// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class NNERuntimeBasicCpu : ModuleRules
{
	public NNERuntimeBasicCpu( ReadOnlyTargetRules Target ) : base( Target )
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange
		(
			new string[] 
			{
				"Core",
				"Engine",
				"NNE",
			}
		);

		PrivateDependencyModuleNames.AddRange
		(
			new string[] 
			{
				"CoreUObject",
			}
		);
	}
}
