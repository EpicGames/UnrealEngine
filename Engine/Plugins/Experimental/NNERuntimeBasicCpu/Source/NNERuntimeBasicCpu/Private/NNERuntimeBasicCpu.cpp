// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeBasicCpu.h"

#include "NNE.h"
#include "NNERuntimeBasicCpuModel.h"
#include "NNEModelData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NNERuntimeBasicCpu)

DEFINE_LOG_CATEGORY(LogNNERuntimeBasicCPU);

// We ask for the memory to be aligned to 64 bytes since this is the
// largest alignment we ask for inside the ModelData for pointers to
// various bits of data.
const uint32 UNNERuntimeBasicCpuImpl::Alignment = 64;

UNNERuntimeBasicCpuImpl::ECanCreateModelDataStatus UNNERuntimeBasicCpuImpl::CanCreateModelData(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) const
{
	if (FileType.Compare("ubnne", ESearchCase::IgnoreCase) != 0)
	{
		return ECanCreateModelDataStatus::FailFileIdNotSupported;
	}

	// We require at least a magic number and version number
	if (FileData.Num() < 2 * sizeof(uint32))
	{
		return ECanCreateModelDataStatus::Fail;
	}

	// Check magic number valid
	const uint32* FileMagicNumber = (const uint32*)&FileData[0 * sizeof(uint32)];
	if (*FileMagicNumber != UE::NNE::RuntimeBasic::FModelCPU::ModelMagicNumber)
	{
		return ECanCreateModelDataStatus::Fail;
	}

	// Check version number valid
	const uint32* FileVersionNumber = (const uint32*)&FileData[1 * sizeof(uint32)];
	if (*FileVersionNumber != UE::NNE::RuntimeBasic::FModelCPU::ModelVersionNumber)
	{
		return ECanCreateModelDataStatus::Fail;
	}

	// Check model do not have additional data
	if (!AdditionalFileData.IsEmpty())
	{
		return ECanCreateModelDataStatus::Fail;
	}

	// Check model is not > 2GB
	if ((TArray<uint8>::SizeType)FileData.Num() != FileData.Num())
	{
		return ECanCreateModelDataStatus::Fail;
	}

	return ECanCreateModelDataStatus::Ok;
}

TSharedPtr<UE::NNE::FSharedModelData> UNNERuntimeBasicCpuImpl::CreateModelData(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform)
{
	if (CanCreateModelData(FileType, FileData, AdditionalFileData, FileId, TargetPlatform) != ECanCreateModelDataStatus::Ok)
	{
		UE_LOGF(LogNNERuntimeBasicCPU, Warning, "Cannot create the model data with id %ls (Filetype: %ls)", *FileId.ToString(EGuidFormats::Digits).ToLower(), *FileType);
		return nullptr;
	}

	// The model data is the same as what is stored in the file however we need to
	// make an owned copy and also ensure it is aligned since we will be creating 
	// pointers to inside.
	uint8* ModelData = (uint8*)FMemory::Malloc(FileData.Num(), Alignment);
	check(ModelData);
	FMemory::Memcpy(ModelData, FileData.GetData(), FileData.Num());

	return MakeShared<UE::NNE::FSharedModelData>(FSharedBuffer::TakeOwnership(ModelData, FileData.Num(), FMemory::Free), Alignment);
}

FString UNNERuntimeBasicCpuImpl::GetModelDataIdentifier(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) const
{
	return FileId.ToString(EGuidFormats::Digits) + "-" + FString::FromInt(UE::NNE::RuntimeBasic::FModelCPU::ModelMagicNumber);
}

UNNERuntimeBasicCpuImpl::ECanCreateModelCPUStatus UNNERuntimeBasicCpuImpl::CanCreateModelCPU(const TObjectPtr<UNNEModelData> ModelData) const
{
	check(ModelData != nullptr);

	TSharedPtr<UE::NNE::FSharedModelData> SharedData = ModelData->GetModelData(GetRuntimeName());

	if (!SharedData.IsValid())
	{
		return ECanCreateModelCPUStatus::Fail;
	}

	TConstArrayView64<uint8> Data = SharedData->GetView();

	// We require at least a magic number and version number
	if (Data.Num() < 2 * sizeof(uint32))
	{
		return ECanCreateModelCPUStatus::Fail;
	}

	// Check magic number valid
	const uint32* FileMagicNumber = (const uint32*)&Data[0 * sizeof(uint32)];
	if (*FileMagicNumber != UE::NNE::RuntimeBasic::FModelCPU::ModelMagicNumber)
	{
		return ECanCreateModelCPUStatus::Fail;
	}

	// Check version number valid
	const uint32* FileVersionNumber = (const uint32*)&Data[1 * sizeof(uint32)];
	if (*FileVersionNumber != UE::NNE::RuntimeBasic::FModelCPU::ModelVersionNumber)
	{
		return ECanCreateModelCPUStatus::Fail;
	}

	return ECanCreateModelCPUStatus::Ok;
}

TSharedPtr<UE::NNE::IModelCPU> UNNERuntimeBasicCpuImpl::CreateModelCPU(const TObjectPtr<UNNEModelData> ModelData)
{
	check(ModelData != nullptr);

	if (CanCreateModelCPU(ModelData) != ECanCreateModelCPUStatus::Ok)
	{
		UE_LOGF(LogNNERuntimeBasicCPU, Warning, "Cannot create a model from the model data with id %ls", *ModelData->GetFileId().ToString(EGuidFormats::Digits));
		return nullptr;
	}

	// Create Model Object
	TSharedPtr<UE::NNE::RuntimeBasic::FModelCPU> Model = MakeShared<UE::NNE::RuntimeBasic::FModelCPU>();
	
	// Store weak pointer to self
	Model->WeakThis = Model.ToWeakPtr();

	// Create Model Data
	Model->ModelData = ModelData->GetModelData(GetRuntimeName());
	check(Model->ModelData != nullptr);

	// Check Model Data Alignment
	checkf(((UPTRINT)(Model->ModelData->GetView().GetData()) % Alignment) == 0,
		TEXT("Model data must be aligned to %i bytes."), Alignment);

	// Load from model data
	uint64 Offset = 0;
	if (Model->SerializationLoad(Offset, Model->ModelData->GetView()))
	{
		return Model;
	}
	else
	{
		return nullptr;
	}
}
