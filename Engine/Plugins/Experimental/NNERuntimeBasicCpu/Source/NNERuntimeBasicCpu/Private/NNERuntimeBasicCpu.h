// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"
#include "NNERuntime.h"
#include "NNERuntimeCPU.h"
#include "UObject/Object.h"

DECLARE_LOG_CATEGORY_EXTERN(LogNNERuntimeBasicCPU, Log, All);

#include "NNERuntimeBasicCpu.generated.h"

#define UE_API NNERUNTIMEBASICCPU_API

/**
 * This plugin is a basic, performant, cross-platform CPU runtime for NNE that supports simple models such as MLPs.
 * 
 * To use this runtime, the custom ".ubnne" file format is used, which can be exported from python using the functions
 * in the provided "nne_runtime_basic_cpu.py" found in the "Content" folder of this plugin. The idea behind this plugin
 * is not to be a general purpose runtime, but rather to provide performant cross-platform implementations for simple
 * CPU models such as MLPs with minimal overhead and memory usage.
 */
UCLASS(MinimalAPI)
class UNNERuntimeBasicCpuImpl : public UObject, public INNERuntime, public INNERuntimeCPU
{
	GENERATED_BODY()

public:

	virtual FString GetRuntimeName() const override { return TEXT("NNERuntimeBasicCpu"); };

	UE_API virtual ECanCreateModelDataStatus CanCreateModelData(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) const override;
	UE_API virtual TSharedPtr<UE::NNE::FSharedModelData> CreateModelData(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) override;
	UE_API virtual FString GetModelDataIdentifier(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) const override;

	UE_API virtual ECanCreateModelCPUStatus CanCreateModelCPU(const TObjectPtr<UNNEModelData> ModelData) const override;
	UE_API virtual TSharedPtr<UE::NNE::IModelCPU> CreateModelCPU(const TObjectPtr<UNNEModelData> ModelData) override;

private:

	static UE_API const uint32 Alignment;
};

#undef UE_API
