// Copyright Epic Games, Inc. All Rights Reserved.

inline float NNERuntimeBasicCPUTanH(const float X, const float Min = -10.0f, const float Max = 10.0f)
{
	const float Exp2X = exp(2.0f * clamp(X, Min, Max));
	return (Exp2X - 1) / (Exp2X + 1);
}

inline float NNERuntimeBasicCPUSigmoid(const float X)
{
	return 1.0f / (1.0f + exp(-X));
}

inline float NNERuntimeBasicCPUGELU(const float X)
{
	return X * NNERuntimeBasicCPUSigmoid(1.702f * X);
}

inline float NNERuntimeBasicCPUSquare(const float X)
{
	return X * X;
}

export void NNERuntimeBasicCPUOperatorNormalize(
	uniform float Output[],
	uniform const float Input[],
	uniform const float Mean[],
	uniform const float Std[],
	uniform const int BatchNum,
	uniform const int InputOutputNum,
	uniform const int OutputStride,
	uniform const int InputStride)
{
	foreach(BatchIdx = 0 ... BatchNum, Idx = 0 ... InputOutputNum)
	{
		Output[BatchIdx * OutputStride + Idx] = (Input[BatchIdx * InputStride + Idx] - Mean[Idx]) / Std[Idx];
	}
}

export void NNERuntimeBasicCPUOperatorDenormalize(
	uniform float Output[],
	uniform const float Input[],
	uniform const float Mean[],
	uniform const float Std[],
	uniform const int BatchNum,
	uniform const int InputOutputNum,
	uniform const int OutputStride,
	uniform const int InputStride)
{
	foreach(BatchIdx = 0 ... BatchNum, Idx = 0 ... InputOutputNum)
	{
		Output[BatchIdx * OutputStride + Idx] = (Input[BatchIdx * InputStride + Idx] * Std[Idx]) + Mean[Idx];
	}
}

export void NNERuntimeBasicCPUOperatorClamp(
	uniform float Output[],
	uniform const float Input[],
	uniform const float MinValues[],
	uniform const float MaxValues[],
	uniform const int BatchNum,
	uniform const int InputOutputNum,
	uniform const int OutputStride,
	uniform const int InputStride)
{
	foreach(BatchIdx = 0 ... BatchNum, Idx = 0 ... InputOutputNum)
	{
		Output[BatchIdx * OutputStride + Idx] = clamp(Input[BatchIdx * InputStride + Idx], MinValues[Idx], MaxValues[Idx]);
	}
}

export void NNERuntimeBasicCPUOperatorLinear(
	uniform float Output[],
	uniform const float Input[],
	uniform const float Weights[],
	uniform const float Biases[],
	uniform const int BatchNum,
	uniform const int ColNum,
	uniform const int RowNum,
	uniform const int OutputStride,
	uniform const int InputStride)
{
	for (uniform int BatchIdx = 0; BatchIdx < BatchNum; BatchIdx++)
	{
		uniform const int BatchInputOffset = BatchIdx * InputStride;
		uniform const int BatchOutputOffset = BatchIdx * OutputStride;

		foreach(ColIdx = 0 ... ColNum)
		{
			Output[BatchOutputOffset + ColIdx] = Biases[ColIdx];
		}

		for (uniform int RowIdx = 0; RowIdx < RowNum; RowIdx++)
		{
			uniform const int RowOffset = RowIdx * ColNum;

			uniform const float Value = Input[BatchInputOffset + RowIdx];

			if (Value != 0.0f)
			{
				foreach(ColIdx = 0 ... ColNum)
				{
					Output[BatchOutputOffset + ColIdx] += Value * Weights[RowOffset + ColIdx];
				}
			}
		}
	}
}

export void NNERuntimeBasicCPUOperatorCompressedLinear(
	uniform float Output[],
	uniform const float Input[],
	uniform const uint16 Weights[],
	uniform const float WeightOffsets[],
	uniform const float WeightScales[],
	uniform const float Biases[],
	uniform const int BatchNum,
	uniform const int ColNum,
	uniform const int RowNum,
	uniform const int OutputStride,
	uniform const int InputStride)
{
	for (uniform int BatchIdx = 0; BatchIdx < BatchNum; BatchIdx++)
	{
		uniform const int BatchInputOffset = BatchIdx * InputStride;
		uniform const int BatchOutputOffset = BatchIdx * OutputStride;

		foreach(ColIdx = 0 ... ColNum)
		{
			Output[BatchOutputOffset + ColIdx] = Biases[ColIdx];
		}

		uniform float WeightOffsetSum = 0.0f;

		for (uniform int RowIdx = 0; RowIdx < RowNum; RowIdx++)
		{
			uniform const int RowOffset = RowIdx * ColNum;

			uniform const float Value = Input[BatchInputOffset + RowIdx];

			WeightOffsetSum += Value * WeightOffsets[RowIdx];

			uniform const float Scale = Value * WeightScales[RowIdx];

			if (Scale != 0.0f)
			{
				foreach(ColIdx = 0 ... ColNum)
				{
					Output[BatchOutputOffset + ColIdx] += Scale * Weights[RowOffset + ColIdx];
				}
			}
		}

		foreach(ColIdx = 0 ... ColNum)
		{
			Output[BatchOutputOffset + ColIdx] += WeightOffsetSum;
		}
	}
}

export void NNERuntimeBasicCPUOperatorMultiLinear(
	uniform float Output[],
	uniform const float Input[],
	uniform const float Weights[],
	uniform const float Biases[],
	uniform const int BatchNum,
	uniform const int BlockNum,
	uniform const int ColNum,
	uniform const int RowNum,
	uniform const int OutputStride,
	uniform const int InputStride)
{
	for (uniform int BatchIdx = 0; BatchIdx < BatchNum; BatchIdx++)
	{
		for (uniform int BlockIdx = 0; BlockIdx < BlockNum; BlockIdx++)
		{
			foreach(ColIdx = 0 ... ColNum)
			{
				Output[BatchIdx * OutputStride + BlockIdx * ColNum + ColIdx] = Biases[BlockIdx * ColNum + ColIdx];
			}

			for (uniform int RowIdx = 0; RowIdx < RowNum; RowIdx++)
			{
				uniform const float Value = Input[BatchIdx * InputStride + BlockIdx * RowNum + RowIdx];

				if (Value != 0.0f)
				{
					foreach (ColIdx = 0 ... ColNum)
					{
						Output[BatchIdx * OutputStride + BlockIdx * ColNum + ColIdx] += Value * Weights[BlockIdx * RowNum * ColNum + RowIdx * ColNum + ColIdx];
					}
				}
			}
		}
	}
}

export void NNERuntimeBasicCPUOperatorCopy(
	uniform float Output[],
	uniform const float Input[],
	uniform const int BatchNum,
	uniform const int Num,
	uniform const int OutputStride,
	uniform const int InputStride)
{
	foreach(BatchIdx = 0 ... BatchNum, Idx = 0 ... Num)
	{
		Output[BatchIdx * OutputStride + Idx] = Input[BatchIdx * InputStride + Idx];
	}
}

export void NNERuntimeBasicCPUOperatorAddInplace(
	uniform float Output[],
	uniform const float Input[],
	uniform const int BatchNum,
	uniform const int Num,
	uniform const int OutputStride,
	uniform const int InputStride)
{
	foreach(BatchIdx = 0 ... BatchNum, Idx = 0 ... Num)
	{
		Output[BatchIdx * OutputStride + Idx] += Input[BatchIdx * InputStride + Idx];
	}
}

export void NNERuntimeBasicCPUOperatorTile(
	uniform float Output[],
	uniform const float Input[],
	uniform const int BatchNum,
	uniform const int Num,
	uniform const int Repeats,
	uniform const int OutputStride,
	uniform const int InputStride)
{
	foreach(BatchIdx = 0 ... BatchNum, Repeat = 0 ... Repeats, Idx = 0 ... Num)
	{
		Output[BatchIdx * OutputStride + Repeat * Num + Idx] = Input[BatchIdx * InputStride + Idx];
	}
}


export void NNERuntimeBasicCPUOperatorReLU(
	uniform float Output[],
	uniform const float Input[],
	uniform const int BatchNum,
	uniform const int Num,
	uniform const int OutputStride,
	uniform const int InputStride)
{
	foreach(BatchIdx = 0 ... BatchNum, Idx = 0 ... Num)
	{
		Output[BatchIdx * OutputStride + Idx] = max(Input[BatchIdx * InputStride + Idx], 0.0f);
	}
}

export void NNERuntimeBasicCPUOperatorELU(
	uniform float Output[],
	uniform const float Input[],
	uniform const int BatchNum,
	uniform const int Num,
	uniform const int OutputStride,
	uniform const int InputStride)
{
	foreach(BatchIdx = 0 ... BatchNum, Idx = 0 ... Num)
	{
		const float Value = Input[BatchIdx * InputStride + Idx];
		Output[BatchIdx * OutputStride + Idx] = select(Value > 0.0f, Value, exp(Value) - 1.0f);
	}
}

export void NNERuntimeBasicCPUOperatorGELU(
	uniform float Output[],
	uniform const float Input[],
	uniform const int BatchNum,
	uniform const int Num,
	uniform const int OutputStride,
	uniform const int InputStride)
{
	foreach(BatchIdx = 0 ... BatchNum, Idx = 0 ... Num)
	{
		Output[BatchIdx * OutputStride + Idx] = NNERuntimeBasicCPUGELU(Input[BatchIdx * InputStride + Idx]);
	}
}

export void NNERuntimeBasicCPUOperatorTanH(
	uniform float Output[],
	uniform const float Input[],
	uniform const int BatchNum,
	uniform const int Num,
	uniform const int OutputStride,
	uniform const int InputStride)
{
	foreach(BatchIdx = 0 ... BatchNum, Idx = 0 ... Num)
	{
		Output[BatchIdx * OutputStride + Idx] = NNERuntimeBasicCPUTanH(Input[BatchIdx * InputStride + Idx]);
	}
}

export void NNERuntimeBasicCPUOperatorPReLU(
	uniform float Output[],
	uniform const float Input[],
	uniform const float Alpha[],
	uniform const int BatchNum,
	uniform const int Num,
	uniform const int OutputStride,
	uniform const int InputStride)
{
	foreach(BatchIdx = 0 ... BatchNum, Idx = 0 ... Num)
	{
		const float Value = Input[BatchIdx * InputStride + Idx];
		Output[BatchIdx * OutputStride + Idx] = select(Value > 0.0f, Value, Alpha[Idx] * Value);
	}
}

export void NNERuntimeBasicCPUOperatorSoftmax(
	uniform float Output[],
	uniform const float Input[],
	uniform const int BatchNum,
	uniform const int Num,
	uniform const int OutputStride,
	uniform const int InputStride)
{
	for (uniform int32 BatchIdx = 0; BatchIdx < BatchNum; BatchIdx++)
	{
		float Maxs = 0.0f;
		foreach (Idx = 0 ... Num)
		{
			Maxs = max(Maxs, Input[BatchIdx * InputStride + Idx]);
		}
		uniform float Max = reduce_max(Maxs);

		float Denoms = 0.0f;
		foreach (Idx = 0 ... Num)
		{
			Denoms += exp(Input[BatchIdx * InputStride + Idx] - Max);
		}
		uniform float Denom = max(reduce_add(Denoms), 1e-8f);

		foreach (Idx = 0 ... Num)
		{
			Output[BatchIdx * OutputStride + Idx] = exp(Input[BatchIdx * InputStride + Idx] - Max) / Denom;
		}
	}
}

export void NNERuntimeBasicCPUOperatorMemoryCellUpdateMemory(
	uniform float Output[],
	uniform const float RememberGate[],
	uniform const float Memory[],
	uniform const float Update[],
	uniform const int BatchSize,
	uniform const int MemorySize,
	uniform const int OutputStride,
	uniform const int RememberGateStride,
	uniform const int MemoryStride,
	uniform const int UpdateStride)
{
	foreach(BatchIdx = 0 ... BatchSize, Idx = 0 ... MemorySize)
	{
		const float Gate = NNERuntimeBasicCPUSigmoid(RememberGate[BatchIdx * RememberGateStride + Idx]);
		const float Prev = Memory[BatchIdx * MemoryStride + Idx];
		const float Targ = NNERuntimeBasicCPUTanH(Update[BatchIdx * UpdateStride + Idx]);

		Output[BatchIdx * OutputStride + Idx] = (1.0f - Gate) * Prev + Gate * Targ;
	}
}

export void NNERuntimeBasicCPUOperatorMemoryCellUpdateOutput(
	uniform float Output[],
	uniform const float PassthroughGate[],
	uniform const float MemoryUpdate[],
	uniform const float InputUpdate[],
	uniform const int BatchSize,
	uniform const int OutputSize,
	uniform const int OutputStride,
	uniform const int PassthroughGateStride,
	uniform const int MemoryUpdateStride,
	uniform const int InputUpdateStride)
{
	foreach(BatchIdx = 0 ... BatchSize, Idx = 0 ... OutputSize)
	{
		const float Gate = NNERuntimeBasicCPUSigmoid(PassthroughGate[BatchIdx * PassthroughGateStride + Idx]);
		const float MemTarg = NNERuntimeBasicCPUTanH(MemoryUpdate[BatchIdx * MemoryUpdateStride + Idx]);
		const float InTarg = NNERuntimeBasicCPUTanH(InputUpdate[BatchIdx * InputUpdateStride + Idx]);

		Output[BatchIdx * OutputStride + Idx] = (1.0f - Gate) * MemTarg + Gate * InTarg;
	}
}

export void NNERuntimeBasicCPUOperatorGRUMaskInput(
	uniform float Masked[],
	uniform const float InputMemory[],
	uniform const float RememberGate[],
	uniform const int BatchSize,
	uniform const int OutputSize,
	uniform const int InputSize,
	uniform const int MaskedStride,
	uniform const int InputMemoryStride,
	uniform const int RememberGateStride)
{
	foreach(BatchIdx = 0 ... BatchSize, Idx = 0 ... InputSize)
	{
		Masked[BatchIdx * MaskedStride + Idx] = InputMemory[BatchIdx * InputMemoryStride + Idx];
	}

	foreach(BatchIdx = 0 ... BatchSize, Idx = 0 ... OutputSize)
	{
		Masked[BatchIdx * MaskedStride + InputSize + Idx] =
			NNERuntimeBasicCPUSigmoid(RememberGate[BatchIdx * RememberGateStride + Idx]) *
			InputMemory[BatchIdx * InputMemoryStride + InputSize + Idx];
	}
}

export void NNERuntimeBasicCPUOperatorGRUCellUpdateOutput(
	uniform float OutputMemory[],
	uniform const float InputMemory[],
	uniform const float UpdateGate[],
	uniform const float ActivationGate[],
	uniform const int BatchSize,
	uniform const int OutputSize,
	uniform const int InputSize,
	uniform const int OutputMemoryStride,
	uniform const int InputMemoryStride,
	uniform const int UpdateGateStride,
	uniform const int ActivationGateStride)
{
	foreach(BatchIdx = 0 ... BatchSize, Idx = 0 ... OutputSize)
	{
		const float MemoryValue = InputMemory[BatchIdx * InputMemoryStride + InputSize + Idx];
		const float UpdateGateValue = NNERuntimeBasicCPUSigmoid(UpdateGate[BatchIdx * UpdateGateStride + Idx]);
		const float ActivationGateValue = NNERuntimeBasicCPUTanH(ActivationGate[BatchIdx * ActivationGateStride + Idx]);
		const float OutputValue = (1.0f - UpdateGateValue) * MemoryValue + UpdateGateValue * ActivationGateValue;

		OutputMemory[BatchIdx * OutputMemoryStride + Idx] = OutputValue;
		OutputMemory[BatchIdx * OutputMemoryStride + OutputSize + Idx] = OutputValue;
	}
}

export void NNERuntimeBasicCPUOperatorAggregateGatherElements(
	uniform float OutputBuffer[],
	uniform const float InputBuffer[],
	uniform const uint32 ElementNums[],
	uniform const uint32 ElementOffsets[],
	uniform const int BatchSize,
	uniform const int ElementSize,
	uniform const int InputStride)
{
	for (uniform int BatchIdx = 0; BatchIdx < BatchSize; BatchIdx++)
	{
		uniform const uint32 ElementNum = ElementNums[BatchIdx];
		uniform const uint32 ElementOffset = ElementOffsets[BatchIdx];

		foreach (ElementIdx = 0 ... ElementNum, Idx = 0 ... ElementSize)
		{
			OutputBuffer[(ElementOffset + ElementIdx) * ElementSize + Idx] = InputBuffer[BatchIdx * InputStride + ElementIdx * ElementSize + Idx];
		}
	}
}

export void NNERuntimeBasicCPUOperatorAggregateInsertOneHot(
	uniform float QueryBuffer[],
	uniform const int Index,
	uniform const int BatchSize,
	uniform const int MaskSize,
	uniform const int QueryBufferStride)
{
	for (uniform int BatchIdx = 0; BatchIdx < BatchSize; BatchIdx++)
	{
		foreach (MaskIdx = 0 ... MaskSize)
		{
			QueryBuffer[BatchIdx * QueryBufferStride + MaskIdx] = 0.0f;
		}

		QueryBuffer[BatchIdx * QueryBufferStride + Index] = 1.0f;
	}
}

export void NNERuntimeBasicCPUOperatorAggregateCountElementNum(
	uniform uint32& TotalElementNum,
	uniform uint32 ElementNums[],
	uniform uint32 ElementOffsets[],
	uniform const float MaskBuffer[],
	uniform const int BatchSize,
	uniform const int MaskSize,
	uniform const int MaskBufferStride)
{
	TotalElementNum = 0;

	for (uniform int BatchIdx = 0; BatchIdx < BatchSize; BatchIdx++)
	{
		uint32 ElementSumAccum = 0;

		foreach (MaskIdx = 0 ... MaskSize)
		{
			if (MaskBuffer[BatchIdx * MaskBufferStride + MaskIdx]) { ElementSumAccum++; }
		}

		uniform uint32 ElementSum = reduce_add(ElementSumAccum);

		ElementOffsets[BatchIdx] = TotalElementNum;
		ElementNums[BatchIdx] = ElementSum;
		TotalElementNum += ElementSum;
	}
}

export void NNERuntimeBasicCPUOperatorAggregateDotProductAttention(
	uniform float Attention[],
	uniform const float Queries[],
	uniform const float Keys[],
	uniform const int ElementNum,
	uniform const int AttentionEncodingSize,
	uniform const int AttentionHeadNum)
{
	for (uniform int ElementIdx = 0; ElementIdx < ElementNum; ElementIdx++)
	{
		for (uniform int HeadIdx = 0; HeadIdx < AttentionHeadNum; HeadIdx++)
		{
			float AttentionAccum = 0.0f;

			foreach (Idx = 0 ... AttentionEncodingSize)
			{
				AttentionAccum += (
					Keys[ElementIdx * AttentionHeadNum * AttentionEncodingSize + HeadIdx * AttentionEncodingSize + Idx] *
					Queries[ElementIdx * AttentionHeadNum * AttentionEncodingSize + HeadIdx * AttentionEncodingSize + Idx]);
			}

			Attention[ElementIdx * AttentionHeadNum + HeadIdx] = reduce_add(AttentionAccum) / sqrt((uniform float)AttentionEncodingSize);
		}
	}
}

export void NNERuntimeBasicCPUOperatorAggregateSoftmaxPlusOneInplace(
	uniform float AttentionMaxs[],
	uniform float AttentionDenoms[],
	uniform float Attention[],
	uniform const uint32 ElementNums[],
	uniform const uint32 ElementOffsets[],
	uniform const int BatchSize,
	uniform const int AttentionHeadNum)
{
	for (uniform int BatchIdx = 0; BatchIdx < BatchSize; BatchIdx++)
	{
		uniform const uint32 ElementNum = ElementNums[BatchIdx];
		uniform const uint32 ElementOffset = ElementOffsets[BatchIdx];

		foreach (HeadIdx = 0 ... AttentionHeadNum)
		{
			AttentionMaxs[HeadIdx] = 0.0f;
			AttentionDenoms[HeadIdx] = 0.0f;
		}

		for (uniform uint32 ElementIdx = ElementOffset; ElementIdx < ElementOffset + ElementNum; ElementIdx++)
		{
			foreach (HeadIdx = 0 ... AttentionHeadNum)
			{
				AttentionMaxs[HeadIdx] = max(AttentionMaxs[HeadIdx], Attention[ElementIdx * AttentionHeadNum + HeadIdx]);
			}
		}

		for (uniform uint32 ElementIdx = ElementOffset; ElementIdx < ElementOffset + ElementNum; ElementIdx++)
		{
			foreach (HeadIdx = 0 ... AttentionHeadNum)
			{
				AttentionDenoms[HeadIdx] += exp(Attention[ElementIdx * AttentionHeadNum + HeadIdx] - AttentionMaxs[HeadIdx]);
			}
		}

		foreach (ElementIdx = ElementOffset ... ElementOffset + ElementNum, HeadIdx = 0 ... AttentionHeadNum)
		{
			Attention[ElementIdx * AttentionHeadNum + HeadIdx] = exp(Attention[ElementIdx * AttentionHeadNum + HeadIdx] - AttentionMaxs[HeadIdx]) / (AttentionDenoms[HeadIdx] + 1.0f);
		}
	}
}

export void NNERuntimeBasicCPUOperatorAggregateAttentionSum(
	uniform float Output[],
	uniform const float Attention[],
	uniform const float Values[],
	uniform const uint32 ElementNums[],
	uniform const uint32 ElementOffsets[],
	uniform const int BatchSize,
	uniform const int EncodingSize,
	uniform const int AttentionHeadNum,
	uniform const int OutputStride)
{
	for (uniform int BatchIdx = 0; BatchIdx < BatchSize; BatchIdx++)
	{
		foreach (HeadIdx = 0 ... AttentionHeadNum, Idx = 0 ... EncodingSize)
		{
			Output[BatchIdx * OutputStride + HeadIdx * EncodingSize + Idx] = 0.0f;
		}

		uniform const uint32 ElementNum = ElementNums[BatchIdx];
		uniform const uint32 ElementOffset = ElementOffsets[BatchIdx];

		for (uniform uint32 ElementIdx = ElementOffset; ElementIdx < ElementOffset + ElementNum; ElementIdx++)
		{
			for (uniform int HeadIdx = 0; HeadIdx < AttentionHeadNum; HeadIdx++)
			{
				uniform const float Scale = Attention[ElementIdx * AttentionHeadNum + HeadIdx];

				if (Scale != 0.0f)
				{
					foreach(Idx = 0 ... EncodingSize)
					{
						Output[BatchIdx * OutputStride + HeadIdx * EncodingSize + Idx] += Scale * Values[ElementIdx * AttentionHeadNum * EncodingSize + HeadIdx * EncodingSize + Idx];
					}
				}
			}
		}
	}
}

export void NNERuntimeBasicCPUOperatorAggregateGatherQueryValueFromSubLayers(
	uniform float QueryBuffer[],
	uniform float KeyBuffer[],
	uniform float ValueBuffer[],
	uniform uint32 ElementAccum[],
	uniform const uint32 ElementOffsets[],
	uniform const float SubLayerQueryBuffer[],
	uniform const float SubLayerKeyBuffer[],
	uniform const float SubLayerValueBuffer[],
	uniform const uint32 SubLayerBatchIndicesBuffer[],
	uniform const int SubLayerBatchIndexNum,
	uniform const int QuerySize,
	uniform const int KeySize,
	uniform const int ValueSize)
{
	for (uniform int ElementIdx = 0; ElementIdx < SubLayerBatchIndexNum; ElementIdx++)
	{
		uniform const uint32 BatchIdx = SubLayerBatchIndicesBuffer[ElementIdx];
		uniform const uint32 ElementOffset = ElementOffsets[BatchIdx] + ElementAccum[BatchIdx];

		foreach (QueryIdx = 0 ... QuerySize)
		{
			QueryBuffer[ElementOffset * QuerySize + QueryIdx] = SubLayerQueryBuffer[ElementIdx * QuerySize + QueryIdx];
		}

		foreach(KeyIdx = 0 ... KeySize)
		{
			KeyBuffer[ElementOffset * KeySize + KeyIdx] = SubLayerKeyBuffer[ElementIdx * KeySize + KeyIdx];
		}

		foreach(ValueIdx = 0 ... ValueSize)
		{
			ValueBuffer[ElementOffset * ValueSize + ValueIdx] = SubLayerValueBuffer[ElementIdx * ValueSize + ValueIdx];
		}

		ElementAccum[BatchIdx]++;
	}
}

export void NNERuntimeBasicCPUOperatorGather(
	uniform float OutputBuffer[],
	uniform const float InputBuffer[],
	uniform const uint32 BatchIndices[],
	uniform const int BatchIndexNum,
	uniform const int InputOutputSize,
	uniform const int OutputStride,
	uniform const int InputStride)
{
	for (uniform int BatchIndexIdx = 0; BatchIndexIdx < BatchIndexNum; BatchIndexIdx++)
	{
		uniform const uint32 SrcBatchIdx = BatchIndices[BatchIndexIdx];

		foreach (Idx = 0 ... InputOutputSize)
		{
			OutputBuffer[BatchIndexIdx * OutputStride + Idx] = InputBuffer[SrcBatchIdx * InputStride + Idx];
		}
	}
}

export void NNERuntimeBasicCPUOperatorScatter(
	uniform float OutputBuffer[],
	uniform const float InputBuffer[],
	uniform const uint32 BatchIndices[],
	uniform const int BatchIndexNum,
	uniform const int InputOutputSize,
	uniform const int OutputStride,
	uniform const int InputStride)
{
	for (uniform int BatchIndexIdx = 0; BatchIndexIdx < BatchIndexNum; BatchIndexIdx++)
	{
		uniform const uint32 DstBatchIdx = BatchIndices[BatchIndexIdx];

		foreach (Idx = 0 ... InputOutputSize)
		{
			OutputBuffer[DstBatchIdx * OutputStride + Idx] = InputBuffer[BatchIndexIdx * InputStride + Idx];
		}
	}
}

export void NNERuntimeBasicCPUOperatorLayerNorm(
	uniform float Output[],
	uniform const float Input[],
	uniform const float Offset[],
	uniform const float Scale[],
	uniform const float Epsilon,
	uniform const int BatchNum,
	uniform const int InputOutputNum,
	uniform const int OutputStride,
	uniform const int InputStride)
{
	for (uniform int BatchIdx = 0; BatchIdx < BatchNum; BatchIdx++)
	{
		float MeanAccum = 0.0f;
		foreach (Idx = 0 ... InputOutputNum)
		{
			MeanAccum += Input[BatchIdx * InputStride + Idx] / InputOutputNum;
		}
		uniform float Mean = reduce_add(MeanAccum);

		float StdAccum = 0.0f;
		foreach (Idx = 0 ... InputOutputNum)
		{
			StdAccum += NNERuntimeBasicCPUSquare(Input[BatchIdx * InputStride + Idx] - Mean) / InputOutputNum;
		}
		uniform float Std = sqrt(reduce_add(StdAccum) + Epsilon);

		foreach (Idx = 0 ... InputOutputNum)
		{
			Output[BatchIdx * OutputStride + Idx] = ((Input[BatchIdx * InputStride + Idx] - Mean) / Std) * Scale[Idx] + Offset[Idx];
		}
	}
}

export void NNERuntimeBasicCPUOperatorLayerFiLM(
	uniform float InputOutput[],
	uniform const float Condition[],
	uniform const int BatchNum,
	uniform const int InputOutputNum,
	uniform const int InputOutputStride,
	uniform const int ConditionStride)
{
	foreach (BatchIdx = 0 ... BatchNum, Idx = 0 ... InputOutputNum)
	{
		InputOutput[BatchIdx * InputOutputStride + Idx] =
			(InputOutput[BatchIdx * InputOutputStride + Idx] *
				Condition[BatchIdx * ConditionStride + Idx]) +
			Condition[BatchIdx * ConditionStride + InputOutputNum + Idx];
	}
}

export void NNERuntimeBasicCPUOperatorConv1d(
	uniform float Output[],
	uniform const float Input[],
	uniform const float Weights[],
	uniform const float Biases[],
	uniform const int BatchNum,
	uniform const int InChannelsNum,
	uniform const int OutChannelsNum,
	uniform const int InputNum,
	uniform const int KernelSize,
	uniform const int Padding,
	uniform const int PaddingMode,
	uniform const int OutputStride,
	uniform const int InputStride)
{
	uniform int OutputNum = InputNum + 2 * Padding - KernelSize + 1;

	for (uniform int BatchIdx = 0; BatchIdx < BatchNum; BatchIdx++)
	{
		uniform const float* InputItr = Input + BatchIdx * InputStride;
		uniform float* OutputItr = Output + BatchIdx * OutputStride;

		for (uniform int OutChannelsIdx = 0; OutChannelsIdx < OutChannelsNum; OutChannelsIdx++)
		{
			uniform const float* WeightsItr = Weights + OutChannelsIdx * InChannelsNum * KernelSize;
			uniform float* OutChannelsItr = OutputItr + OutChannelsIdx * OutputNum;
			uniform float Bias = Biases[OutChannelsIdx];

			foreach (OutputIdx = 0 ... OutputNum)
			{
				float Sum = Bias;

				for (uniform int InChannelsIdx = 0; InChannelsIdx < InChannelsNum; InChannelsIdx++)
				{
					uniform const float* WeightsInChannelsItr = WeightsItr + InChannelsIdx * KernelSize;
					uniform const float* InChannelsItr = InputItr + InChannelsIdx * InputNum;

					if (PaddingMode == 0) //Zeros
					{
						for (uniform int KernelIdx = 0; KernelIdx < KernelSize; KernelIdx++)
						{
							int Idx = OutputIdx + KernelIdx - Padding;
							float Val = (Idx < 0 || Idx >= InputNum) ? 0.0f : InChannelsItr[Idx];
							Sum += Val * WeightsInChannelsItr[KernelIdx];
						}
					}
					else if (PaddingMode == 1) //Circular
					{
						for (uniform int KernelIdx = 0; KernelIdx < KernelSize; KernelIdx++)
						{
							int Idx = OutputIdx + KernelIdx - Padding;
							#pragma ignore warning
							Idx = Idx % InputNum;
							if (Idx < 0) {Idx += InputNum;}
							#pragma ignore warning
							Sum += InChannelsItr[Idx] * WeightsInChannelsItr[KernelIdx];
						}
					}
				}
				OutChannelsItr[OutputIdx] = Sum;
			}
		}
	}
}


export void NNERuntimeBasicCPUOperatorConv2d(
	uniform float Output[],
	uniform const float Input[],
	uniform const float Weights[],
	uniform const float Biases[],
	uniform const int BatchNum,
	uniform const int InChannelsNum,
	uniform const int OutChannelsNum,
	uniform const int InputHeight,
	uniform const int InputWidth,
	uniform const int KernelSize,
	uniform const int Stride,
	uniform const int Padding,
	uniform const int PaddingMode,
	uniform const int OutputStride,
	uniform const int InputStride)
{
	uniform int OutputHeight = (InputHeight + 2 * Padding - KernelSize) / Stride + 1;
	uniform int OutputWidth = (InputWidth + 2 * Padding - KernelSize) / Stride + 1;

	for (uniform int BatchIdx = 0; BatchIdx < BatchNum; BatchIdx++)
	{
		uniform const float* InputBatchItr = Input + BatchIdx * InputStride;
		uniform float* OutputBatchItr = Output + BatchIdx * OutputStride;

		for (uniform int OutChannelsIdx = 0; OutChannelsIdx < OutChannelsNum; OutChannelsIdx++)
		{
			uniform const float* WeightsItr = Weights + OutChannelsIdx * InChannelsNum * KernelSize * KernelSize;
			uniform float Bias = Biases[OutChannelsIdx];

			foreach (OutputHeightIdx = 0 ... OutputHeight, OutputWidthIdx = 0 ... OutputWidth)
			{
                float Sum = Bias;

                for (uniform int InChannelsIdx = 0; InChannelsIdx < InChannelsNum; InChannelsIdx++)
                {
                    uniform const float* WeightsInChannelsItr = WeightsItr + InChannelsIdx * KernelSize * KernelSize;
                    uniform const float* InChannelsItr = InputBatchItr + InChannelsIdx * InputHeight * InputWidth;

                    for (uniform int KernelHeight = 0; KernelHeight < KernelSize; KernelHeight++)
                    {
                        for (uniform int KernelWidth = 0; KernelWidth < KernelSize; KernelWidth++)
                        {
                            int HeightIdx = OutputHeightIdx * Stride + KernelHeight - Padding;
                            int WidthIdx = OutputWidthIdx * Stride + KernelWidth - Padding;
                            
                            float Val = 0.0f;
                            if (HeightIdx >= 0 && HeightIdx < InputHeight && WidthIdx >= 0 && WidthIdx < InputWidth)
                            {
								#pragma ignore warning
                                Val = InChannelsItr[HeightIdx * InputWidth + WidthIdx];
                            }
                            Sum += Val * WeightsInChannelsItr[KernelHeight * KernelSize + KernelWidth];
                        }
                    }
                }
                OutputBatchItr[OutChannelsIdx * OutputHeight * OutputWidth + OutputHeightIdx * OutputWidth + OutputWidthIdx] = Sum;
			}
		}
	}
}