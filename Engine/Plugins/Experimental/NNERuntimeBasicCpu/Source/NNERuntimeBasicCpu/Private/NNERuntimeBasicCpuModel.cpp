// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeBasicCpuModel.h"

#include "NNE.h"
#include "NNERuntimeBasicCpu.h"
#include "NNERuntimeBasicCpuBuilder.h"

#define NNE_RUNTIME_BASIC_ENABLE_ISPC INTEL_ISPC
//#define NNE_RUNTIME_BASIC_ENABLE_ISPC 0

//#define NNE_RUNTIME_BASIC_ENABLE_NAN_CHECK (!UE_BUILD_SHIPPING)
#define NNE_RUNTIME_BASIC_ENABLE_NAN_CHECK 0

#define NNE_RUNTIME_BASIC_ENABLE_PROFILE (!UE_BUILD_SHIPPING)
//#define NNE_RUNTIME_BASIC_ENABLE_PROFILE 0

#define NNE_RUNTIME_BASIC_ENABLE_ALIASING_CHECK (!UE_BUILD_SHIPPING)
//#define NNE_RUNTIME_BASIC_ENABLE_ALIASING_CHECK 0

#if NNE_RUNTIME_BASIC_ENABLE_PROFILE
#define NNE_RUNTIME_BASIC_TRACE_SCOPE(...) TRACE_CPUPROFILER_EVENT_SCOPE(__VA_ARGS__)
#else
#define NNE_RUNTIME_BASIC_TRACE_SCOPE(...)
#endif

#if NNE_RUNTIME_BASIC_ENABLE_ISPC
#include "NNERuntimeBasicCpu.ispc.generated.h"
#endif

#if NNE_RUNTIME_BASIC_ENABLE_ALIASING_CHECK
#define NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZE(X, Y, Size) check(X == nullptr || Y == nullptr || Size == 0 || (X != Y));
#define NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZES(X, Y, XSize, YSize) check(X == nullptr || Y == nullptr || XSize == 0 || YSize == 0 || (X != Y));
#else
#define NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZE(...)
#define NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZES(...)
#endif

namespace UE::NNE::RuntimeBasic
{
	namespace Private
	{
		//--------------------------------------------------------------------------
		// Serialization Helpers
		//--------------------------------------------------------------------------

		namespace Serialization
		{
			//--------------------------------------------------------------------------

			static inline void NanCheck(TConstArrayView<float> In)
			{
#if NNE_RUNTIME_BASIC_ENABLE_NAN_CHECK
				for (int32 Idx = 0; Idx < In.Num(); Idx++)
				{
					checkf(FMath::IsFinite(In[Idx]) && In[Idx] != MAX_flt && In[Idx] != -MAX_flt,
						TEXT("Invalid value %f found in Array at index %i"), In[Idx], Idx);
				}
#endif
			}

			//--------------------------------------------------------------------------

			static inline void Align(uint64& InOutOffset, const uint32 Alignment)
			{
				InOutOffset = ((InOutOffset + Alignment - 1) / Alignment) * Alignment;
			}

			//--------------------------------------------------------------------------

			static inline void Size(uint64& InOutOffset, const uint32& In)
			{
				Align(InOutOffset, sizeof(uint32));
				InOutOffset += sizeof(uint32);
			}

			static inline void Size(uint64& InOutOffset, const float& In)
			{
				Align(InOutOffset, sizeof(float));
				InOutOffset += sizeof(float);
			}

			static inline void Size(uint64& InOutOffset, const TConstArrayView<float> In)
			{
				Align(InOutOffset, 64);
				InOutOffset += In.Num() * sizeof(float);
			}

			static inline void Size(uint64& InOutOffset, const TConstArrayView<uint16> In)
			{
				Align(InOutOffset, 64);
				InOutOffset += In.Num() * sizeof(uint16);
			}

			static inline void Size(uint64& InOutOffset, const TConstArrayView<uint32> In)
			{
				Align(InOutOffset, 64);
				InOutOffset += In.Num() * sizeof(uint32);
			}

			static inline void Size(uint64& InOutOffset, const TSharedPtr<ILayer>& InLayer);
			static inline void Size(uint64& InOutOffset, const TConstArrayView<TSharedPtr<ILayer>> InLayers);

			//--------------------------------------------------------------------------

			static inline void Load(uint64& InOutOffset, uint32& Out, TConstArrayView<uint8> Data)
			{
				Align(InOutOffset, sizeof(uint32));
				Out = *((uint32*)(Data.GetData() + InOutOffset));
				InOutOffset += sizeof(uint32);
			}

			static inline void Load(uint64& InOutOffset, float& Out, TConstArrayView<uint8> Data)
			{
				Align(InOutOffset, sizeof(float));
				Out = *((float*)(Data.GetData() + InOutOffset));
				InOutOffset += sizeof(float);
			}

			static inline void Load(uint64& InOutOffset, TConstArrayView<float>& Out, TConstArrayView<uint8> Data, int32 Size)
			{
				Align(InOutOffset, 64);
				Out = MakeArrayView<const float>((const float*)(Data.GetData() + InOutOffset), Size);
				InOutOffset += Size * sizeof(float);
				NanCheck(Out);
			}

			static inline void Load(uint64& InOutOffset, TConstArrayView<uint16>& Out, TConstArrayView<uint8> Data, int32 Size)
			{
				Align(InOutOffset, 64);
				Out = MakeArrayView<const uint16>((const uint16*)(Data.GetData() + InOutOffset), Size);
				InOutOffset += Size * sizeof(uint16);
			}

			static inline void Load(uint64& InOutOffset, TConstArrayView<uint32>& Out, TConstArrayView<uint8> Data, int32 Size)
			{
				Align(InOutOffset, 64);
				Out = MakeArrayView<const uint32>((const uint32*)(Data.GetData() + InOutOffset), Size);
				InOutOffset += Size * sizeof(uint32);
			}

			static inline void Load(uint64& InOutOffset, TSharedPtr<ILayer>& OutLayer, TConstArrayView<uint8> Data);
			static inline void Load(uint64& InOutOffset, TArrayView<TSharedPtr<ILayer>> OutLayers, TConstArrayView<uint8> Data);

			//--------------------------------------------------------------------------

			static inline void Save(uint64& InOutOffset, const uint32 In, TArrayView<uint8> Data)
			{
				Align(InOutOffset, sizeof(uint32));
				*((uint32*)(Data.GetData() + InOutOffset)) = In;
				InOutOffset += sizeof(uint32);
			}

			static inline void Save(uint64& InOutOffset, const float In, TArrayView<uint8> Data)
			{
				Align(InOutOffset, sizeof(float));
				*((float*)(Data.GetData() + InOutOffset)) = In;
				InOutOffset += sizeof(float);
			}

			static inline void Save(uint64& InOutOffset, TConstArrayView<float> In, TArrayView<uint8> Data)
			{
				NanCheck(In);
				Align(InOutOffset, 64);
				FMemory::Memcpy(Data.GetData() + InOutOffset, In.GetData(), In.Num() * sizeof(float));
				InOutOffset += In.Num() * sizeof(float);
			}

			static inline void Save(uint64& InOutOffset, TConstArrayView<uint16> In, TArrayView<uint8> Data)
			{
				Align(InOutOffset, 64);
				FMemory::Memcpy(Data.GetData() + InOutOffset, In.GetData(), In.Num() * sizeof(uint16));
				InOutOffset += In.Num() * sizeof(uint16);
			}

			static inline void Save(uint64& InOutOffset, TConstArrayView<uint32> In, TArrayView<uint8> Data)
			{
				Align(InOutOffset, 64);
				FMemory::Memcpy(Data.GetData() + InOutOffset, In.GetData(), In.Num() * sizeof(uint32));
				InOutOffset += In.Num() * sizeof(uint32);
			}

			static inline void Save(uint64& InOutOffset, const TSharedPtr<ILayer>& InLayer, TArrayView<uint8> Data);
			static inline void Save(uint64& InOutOffset, const TConstArrayView<TSharedPtr<ILayer>> InLayers, TArrayView<uint8> Data);
		}

		//--------------------------------------------------------------------------
		// Basic Mathematical Functions
		//--------------------------------------------------------------------------

		static inline float Sigmoid(const float X)
		{
			return 1.0f / (1.0f + FMath::Exp(-X));
		}

		//--------------------------------------------------------------------------
		// Operators
		//--------------------------------------------------------------------------

		static inline void OperatorNanCheck(
			const float* RESTRICT InputOutput, 
			const uint32 BatchSize,
			const uint32 InputOutputSize,
			const uint32 InputOutputStride)
		{
#if NNE_RUNTIME_BASIC_ENABLE_NAN_CHECK

			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::OperatorNanCheck);

			for (uint32 BatchIdx = 0; BatchIdx < BatchSize; BatchIdx++)
			{
				for (uint32 Idx = 0; Idx < InputOutputSize; Idx++)
				{
					const float Value = InputOutput[BatchIdx * InputOutputStride + Idx];
					checkf(FMath::IsFinite(Value) && Value != MAX_flt && Value != -MAX_flt,
						TEXT("Invalid value %f found in Batch %i, Value %i"), Value, BatchIdx, Idx);
				}
			}
#endif
		}

		static inline void OperatorCopy(
			float* RESTRICT Output,
			const float* RESTRICT Input,
			const uint32 BatchSize,
			const uint32 InputOutputSize,
			const uint32 OutputStride,
			const uint32 InputStride)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::OperatorCopy);

			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZE(Output, Input, InputOutputSize);

#if NNE_RUNTIME_BASIC_ENABLE_ISPC
			ispc::NNERuntimeBasicCPUOperatorCopy(
				Output,
				Input,
				BatchSize,
				InputOutputSize,
				OutputStride,
				InputStride);
#else
			for (uint32 BatchIdx = 0; BatchIdx < BatchSize; BatchIdx++)
			{
				for (uint32 Idx = 0; Idx < InputOutputSize; Idx++)
				{
					Output[BatchIdx * OutputStride + Idx] = Input[BatchIdx * InputStride + Idx];
				}
			}
#endif
		}

		static inline void OperatorAddInplace(
			float* RESTRICT Output,
			const float* RESTRICT Input,
			const uint32 BatchSize,
			const uint32 InputOutputSize,
			const uint32 OutputStride,
			const uint32 InputStride)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::OperatorAddInplace);

			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZE(Output, Input, InputOutputSize);

#if NNE_RUNTIME_BASIC_ENABLE_ISPC
			ispc::NNERuntimeBasicCPUOperatorAddInplace(
				Output,
				Input,
				BatchSize,
				InputOutputSize,
				OutputStride,
				InputStride);
#else
			for (uint32 BatchIdx = 0; BatchIdx < BatchSize; BatchIdx++)
			{
				for (uint32 Idx = 0; Idx < InputOutputSize; Idx++)
				{
					Output[BatchIdx * OutputStride + Idx] += Input[BatchIdx * InputStride + Idx];
				}
			}
#endif
		}

		static inline void OperatorTile(
			float* RESTRICT Output,
			const float* RESTRICT Input,
			const uint32 BatchSize,
			const uint32 InputSize,
			const uint32 Repeats,
			const uint32 OutputStride,
			const uint32 InputStride)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::OperatorTile);

			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZES(Output, Input, Repeats * InputSize, InputSize);

#if NNE_RUNTIME_BASIC_ENABLE_ISPC
			ispc::NNERuntimeBasicCPUOperatorTile(
				Output,
				Input,
				BatchSize,
				InputSize,
				Repeats,
				OutputStride,
				InputStride);
#else
			for (uint32 BatchIdx = 0; BatchIdx < BatchSize; BatchIdx++)
			{
				for (uint32 Repeat = 0; Repeat < Repeats; Repeat++)
				{
					for (uint32 Idx = 0; Idx < InputSize; Idx++)
					{
						Output[BatchIdx * OutputStride + Repeat * InputSize + Idx] = Input[BatchIdx * InputStride + Idx];
					}
				}
			}
#endif
		}

		static inline void OperatorNormalize(
			float* RESTRICT Output,
			const float* RESTRICT Input,
			const float* RESTRICT Mean,
			const float* RESTRICT Std,
			const uint32 BatchSize,
			const uint32 InputOutputSize,
			const uint32 OutputStride,
			const uint32 InputStride)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::OperatorNormalize);

			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZE(Output, Input, InputOutputSize);

#if NNE_RUNTIME_BASIC_ENABLE_ISPC
			ispc::NNERuntimeBasicCPUOperatorNormalize(
				Output,
				Input,
				Mean,
				Std,
				BatchSize,
				InputOutputSize,
				OutputStride,
				InputStride);
#else
			for (uint32 BatchIdx = 0; BatchIdx < BatchSize; BatchIdx++)
			{
				for (uint32 Idx = 0; Idx < InputOutputSize; Idx++)
				{
					Output[BatchIdx * OutputStride + Idx] = (Input[BatchIdx * InputStride + Idx] - Mean[Idx]) / Std[Idx];
				}
			}
#endif
		}

		static inline void OperatorDenormalize(
			float* RESTRICT Output,
			const float* RESTRICT Input,
			const float* RESTRICT Mean,
			const float* RESTRICT Std,
			const uint32 BatchSize,
			const uint32 InputOutputSize,
			const uint32 OutputStride,
			const uint32 InputStride)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::OperatorDenormalize);

			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZE(Output, Input, InputOutputSize);

#if NNE_RUNTIME_BASIC_ENABLE_ISPC
			ispc::NNERuntimeBasicCPUOperatorDenormalize(
				Output,
				Input,
				Mean,
				Std,
				BatchSize,
				InputOutputSize,
				OutputStride,
				InputStride);
#else
			for (uint32 BatchIdx = 0; BatchIdx < BatchSize; BatchIdx++)
			{
				for (uint32 Idx = 0; Idx < InputOutputSize; Idx++)
				{
					Output[BatchIdx * OutputStride + Idx] = (Input[BatchIdx * InputStride + Idx] * Std[Idx]) + Mean[Idx];
				}
			}
#endif
		}

		static inline void OperatorClamp(
			float* RESTRICT Output,
			const float* RESTRICT Input,
			const float* RESTRICT MinValues,
			const float* RESTRICT MaxValues,
			const uint32 BatchSize,
			const uint32 InputOutputSize,
			const uint32 OutputStride,
			const uint32 InputStride)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::OperatorClamp);

			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZE(Output, Input, InputOutputSize);

#if NNE_RUNTIME_BASIC_ENABLE_ISPC
			ispc::NNERuntimeBasicCPUOperatorClamp(
				Output,
				Input,
				MinValues,
				MaxValues,
				BatchSize,
				InputOutputSize,
				OutputStride,
				InputStride);
#else
			for (uint32 BatchIdx = 0; BatchIdx < BatchSize; BatchIdx++)
			{
				for (uint32 Idx = 0; Idx < InputOutputSize; Idx++)
				{
					Output[BatchIdx * OutputStride + Idx] = FMath::Clamp(Input[BatchIdx * InputStride + Idx], MinValues[Idx], MaxValues[Idx]);
				}
			}
#endif
		}

		static inline void OperatorConv1d(
			float* RESTRICT Output, 
			const float* RESTRICT Input,
			const float* RESTRICT Weights,
			const float* RESTRICT Biases,
			const uint32 BatchSize,
			const uint32 InChannels,
			const uint32 OutChannels,
			const uint32 InputLength,
			const uint32 KernelSize,
			const uint32 Padding,
			const FModelBuilder::EPaddingMode PaddingMode,
			const uint32 OutputStride,
			const uint32 InputStride)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::OperatorConv1d);
			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZES(Output, Input, (InputLength + 2 * Padding - KernelSize + 1) * OutChannels, InputLength * InChannels);

#if NNE_RUNTIME_BASIC_ENABLE_ISPC
			ispc::NNERuntimeBasicCPUOperatorConv1d(
				Output,
				Input,
				Weights,
				Biases,
				BatchSize,
				InChannels,
				OutChannels,
				InputLength,
				KernelSize,
				Padding,
				static_cast<uint32>(PaddingMode),
				OutputStride,
				InputStride);
#else
			check(KernelSize <= InputLength + 2 * Padding);
			const uint32 OutputLength = InputLength + 2 * Padding - KernelSize + 1;
			for (uint32 BatchIdx = 0; BatchIdx < BatchSize; BatchIdx++)
			{
				const float* InputItr = Input + BatchIdx * InputStride;
				float* OutputItr = Output + BatchIdx * OutputStride;
				
				for (uint32 OutChannelsIdx = 0; OutChannelsIdx < OutChannels; OutChannelsIdx++)
				{
					const float* WeightsItr = Weights + OutChannelsIdx * InChannels * KernelSize;
					float* OutChannelsItr = OutputItr + OutChannelsIdx * OutputLength;
					
					for (uint32 OutputIdx = 0; OutputIdx < OutputLength; OutputIdx++)
					{
						float Sum = Biases[OutChannelsIdx];

						for (uint32 InChannelsIdx = 0; InChannelsIdx < InChannels; InChannelsIdx++)
						{
							const float* WeightsInChannelsItr = WeightsItr + InChannelsIdx * KernelSize;
							const float* InChannelsItr = InputItr + InChannelsIdx * InputLength;
							
							for (uint32 KernelIdx = 0; KernelIdx < KernelSize; KernelIdx++)
							{
								int32 Idx = static_cast<int32_t>(OutputIdx) + static_cast<int32_t>(KernelIdx) - static_cast<int32_t>(Padding);
								float Val;

								switch (PaddingMode)
								{
								case FModelBuilder::EPaddingMode::Circular:
								{
									Idx = Idx % static_cast<int32_t>(InputLength);
									if (Idx < 0)
									{
										Idx += static_cast<int32_t>(InputLength);
									}
									Val = InChannelsItr[Idx];
									break;
								}
								case FModelBuilder::EPaddingMode::Zeros:
								{
									if (Idx < 0 or Idx >= static_cast<int32_t>(InputLength))
									{
										Val = 0.f;
									}
									else
									{
										Val = InChannelsItr[Idx];
									}
									break;
								}
								default:
								{
									checkNoEntry();
								}
								}
								Sum += Val * WeightsInChannelsItr[KernelIdx];
							}
						}
						OutChannelsItr[OutputIdx] = Sum;
					}
				}
			}
#endif
		}

		static inline void OperatorConv2d(
			float* RESTRICT Output,
			const float* RESTRICT Input,
			const float* RESTRICT Weights,
			const float* RESTRICT Biases,
			const uint32 BatchSize,
			const uint32 InChannels,
			const uint32 OutChannels,
			const uint32 InputHeight,
			const uint32 InputWidth,
			const uint32 KernelSize,
			const uint32 Stride,
			const uint32 Padding,
			const FModelBuilder::EPaddingMode PaddingMode,
			const uint32 OutputStride,
			const uint32 InputStride)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::OperatorConv2d);
			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZES(Output, Input, ((InputHeight + 2 * Padding - KernelSize) / Stride + 1) * ((InputWidth + 2 * Padding - KernelSize) / Stride + 1) * OutChannels, InputHeight * InputWidth * InChannels);

			check(PaddingMode == FModelBuilder::EPaddingMode::Zeros);

#if NNE_RUNTIME_BASIC_ENABLE_ISPC
			ispc::NNERuntimeBasicCPUOperatorConv2d(
				Output,
				Input,
				Weights,
				Biases,
				BatchSize,
				InChannels,
				OutChannels,
				InputHeight,
				InputWidth,
				KernelSize,
				Stride,
				Padding,
				static_cast<uint32>(PaddingMode),
				OutputStride,
				InputStride);
#else
			const uint32 OutputHeight = (InputHeight + 2 * Padding - KernelSize) / Stride + 1;
			const uint32 OutputWidth = (InputWidth + 2 * Padding - KernelSize) / Stride + 1;

			for (uint32 BatchIdx = 0; BatchIdx < BatchSize; BatchIdx++)
			{
				const float* InputBatchItr = Input + BatchIdx * InputStride;
				float* OutputBatchItr = Output + BatchIdx * OutputStride;

				for (uint32 OutChannelsIdx = 0; OutChannelsIdx < OutChannels; OutChannelsIdx++)
				{
					const float Bias = Biases[OutChannelsIdx];
					const float* WeightsItr = Weights + OutChannelsIdx * InChannels * KernelSize * KernelSize;

					for (uint32 OutputHeightIdx = 0; OutputHeightIdx < OutputHeight; OutputHeightIdx++)
					{
						for (uint32 OutputWidthIdx = 0; OutputWidthIdx < OutputWidth; OutputWidthIdx++)
						{	
							float Sum = Bias;

							for (uint32 InChannelsIdx = 0; InChannelsIdx < InChannels; InChannelsIdx++)
							{
								const float* WeightsInChannelsItr = WeightsItr + InChannelsIdx * KernelSize * KernelSize;
								const float* InChannelsItr = InputBatchItr + InChannelsIdx * InputHeight * InputWidth;

								for (uint32 KernelHeightIdx = 0; KernelHeightIdx < KernelSize; KernelHeightIdx++)
								{
									for (uint32 KernelWidthIdx = 0; KernelWidthIdx < KernelSize; KernelWidthIdx++)
									{
										int32 HeightIdx = OutputHeightIdx * Stride + KernelHeightIdx - Padding;
										int32 WidthIdx = OutputWidthIdx * Stride + KernelWidthIdx - Padding;

										float Val = InChannelsItr[HeightIdx * InputWidth + WidthIdx];
										Sum += Val * WeightsInChannelsItr[KernelHeightIdx * KernelSize + KernelWidthIdx];
									}
								}
							}
							OutputBatchItr[OutChannelsIdx * OutputHeight * OutputWidth + OutputHeightIdx * OutputWidth + OutputWidthIdx] = Sum;
						}
					}
				}
			}
#endif
		}

		static inline void OperatorLinear(
			float* RESTRICT Output,
			const float* RESTRICT Input,
			const float* RESTRICT Weights,
			const float* RESTRICT Biases,
			const uint32 BatchSize,
			const uint32 OutputSize,
			const uint32 InputSize,
			const uint32 OutputStride,
			const uint32 InputStride)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::OperatorLinear);

			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZES(Output, Input, OutputSize, InputSize);

#if NNE_RUNTIME_BASIC_ENABLE_ISPC
			ispc::NNERuntimeBasicCPUOperatorLinear(
				Output,
				Input,
				Weights,
				Biases,
				BatchSize,
				OutputSize,
				InputSize,
				OutputStride,
				InputStride);
#else
			for (uint32 BatchIdx = 0; BatchIdx < BatchSize; BatchIdx++)
			{
				for (uint32 ColIdx = 0; ColIdx < OutputSize; ColIdx++)
				{
					Output[BatchIdx * OutputStride + ColIdx] = Biases[ColIdx];
				}

				for (uint32 RowIdx = 0; RowIdx < InputSize; RowIdx++)
				{
					const float Value = Input[BatchIdx * InputStride + RowIdx];

					if (Value != 0.0)
					{
						for (uint32 ColIdx = 0; ColIdx < OutputSize; ColIdx++)
						{
							Output[BatchIdx * OutputStride + ColIdx] += Value * Weights[RowIdx * OutputSize + ColIdx];
						}
					}
				}
			}
#endif
		}

		static inline void OperatorCompressedLinear(
			float* RESTRICT Output,
			const float* RESTRICT Input,
			const uint16* RESTRICT Weights,
			const float* RESTRICT WeightOffsets,
			const float* RESTRICT WeightScales,
			const float* RESTRICT Biases,
			const uint32 BatchSize,
			const uint32 OutputSize,
			const uint32 InputSize,
			const uint32 OutputStride,
			const uint32 InputStride)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::OperatorCompressedLinear);

			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZES(Output, Input, OutputSize, InputSize);

#if NNE_RUNTIME_BASIC_ENABLE_ISPC
			ispc::NNERuntimeBasicCPUOperatorCompressedLinear(
				Output,
				Input,
				Weights,
				WeightOffsets,
				WeightScales,
				Biases,
				BatchSize,
				OutputSize,
				InputSize,
				OutputStride,
				InputStride);
#else
			for (uint32 BatchIdx = 0; BatchIdx < BatchSize; BatchIdx++)
			{
				for (uint32 ColIdx = 0; ColIdx < OutputSize; ColIdx++)
				{
					Output[BatchIdx * OutputStride + ColIdx] = Biases[ColIdx];
				}

				for (uint32 RowIdx = 0; RowIdx < InputSize; RowIdx++)
				{
					const float Value = Input[BatchIdx * InputStride + RowIdx];

					if (Value != 0.0)
					{
						const float Offset = Value * WeightOffsets[RowIdx];
						const float Scales = Value * WeightScales[RowIdx];

						for (uint32 ColIdx = 0; ColIdx < OutputSize; ColIdx++)
						{
							Output[BatchIdx * OutputStride + ColIdx] += Scales * Weights[RowIdx * OutputSize + ColIdx] + Offset;
						}
					}
				}
			}
#endif
		}

		static inline void OperatorMultiLinear(
			float* RESTRICT Output,
			const float* RESTRICT Input,
			const float* RESTRICT Weights,
			const float* RESTRICT Biases,
			const uint32 BatchSize,
			const uint32 BlockNum,
			const uint32 OutputSize,
			const uint32 InputSize,
			const uint32 OutputStride,
			const uint32 InputStride)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::OperatorMultiLinear);

			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZES(Output, Input, OutputSize, InputSize);

			// For this function ispc generates slightly less efficient code than the naive C++ implementation so we
			// don't bother calling out to the ispc version even if it is available

			for (uint32 BatchIdx = 0; BatchIdx < BatchSize; BatchIdx++)
			{
				for (uint32 BlockIdx = 0; BlockIdx < BlockNum; BlockIdx++)
				{
					for (uint32 ColIdx = 0; ColIdx < OutputSize; ColIdx++)
					{
						Output[BatchIdx * OutputStride + BlockIdx * OutputSize + ColIdx] = Biases[BlockIdx * OutputSize + ColIdx];
					}

					for (uint32 RowIdx = 0; RowIdx < InputSize; RowIdx++)
					{
						const float Value = Input[BatchIdx * InputStride + BlockIdx * InputSize + RowIdx];

						if (Value != 0.0)
						{
							for (uint32 ColIdx = 0; ColIdx < OutputSize; ColIdx++)
							{
								Output[BatchIdx * OutputStride + BlockIdx * OutputSize + ColIdx] += Value * Weights[BlockIdx * InputSize * OutputSize + RowIdx * OutputSize + ColIdx];
							}
						}
					}
				}
			}
		}

		static inline void OperatorReLU(
			float* RESTRICT Output,
			const float* RESTRICT Input,
			const uint32 BatchSize,
			const uint32 InputOutputSize,
			const uint32 OutputStride,
			const uint32 InputStride)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::OperatorReLU);

			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZE(Output, Input, InputOutputSize);

#if NNE_RUNTIME_BASIC_ENABLE_ISPC
			ispc::NNERuntimeBasicCPUOperatorReLU(
				Output,
				Input,
				BatchSize,
				InputOutputSize,
				OutputStride,
				InputStride);
#else
			for (uint32 BatchIdx = 0; BatchIdx < BatchSize; BatchIdx++)
			{
				for (uint32 Idx = 0; Idx < InputOutputSize; Idx++)
				{
					Output[BatchIdx * OutputStride + Idx] = FMath::Max(Input[BatchIdx * InputStride + Idx], 0.0f);
				}
			}
#endif
		}

		static inline void OperatorELU(
			float* RESTRICT Output,
			const float* RESTRICT Input,
			const uint32 BatchSize,
			const uint32 InputOutputSize,
			const uint32 OutputStride,
			const uint32 InputStride)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::OperatorELU);

			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZE(Output, Input, InputOutputSize);

#if NNE_RUNTIME_BASIC_ENABLE_ISPC
			ispc::NNERuntimeBasicCPUOperatorELU(
				Output,
				Input,
				BatchSize,
				InputOutputSize,
				OutputStride,
				InputStride);
#else
			for (uint32 BatchIdx = 0; BatchIdx < BatchSize; BatchIdx++)
			{
				for (uint32 Idx = 0; Idx < InputOutputSize; Idx++)
				{
					const float Value = Input[BatchIdx * InputStride + Idx];
					Output[BatchIdx * OutputStride + Idx] = Value > 0.0f ? Value : FMath::InvExpApprox(-Value) - 1.0f;
				}
			}
#endif
		}

		static inline float GELU(const float X)
		{
			return X * Sigmoid(1.702f * X);
		}

		static inline void OperatorGELU(
			float* RESTRICT Output,
			const float* RESTRICT Input,
			const uint32 BatchSize,
			const uint32 InputOutputSize,
			const uint32 OutputStride,
			const uint32 InputStride)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::OperatorGELU);

			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZE(Output, Input, InputOutputSize);

#if NNE_RUNTIME_BASIC_ENABLE_ISPC
			ispc::NNERuntimeBasicCPUOperatorGELU(
				Output,
				Input,
				BatchSize,
				InputOutputSize,
				OutputStride,
				InputStride);
#else
			for (uint32 BatchIdx = 0; BatchIdx < BatchSize; BatchIdx++)
			{
				for (uint32 Idx = 0; Idx < InputOutputSize; Idx++)
				{
					Output[BatchIdx * OutputStride + Idx] = GELU(Input[BatchIdx * InputStride + Idx]);
				}
			}
#endif
		}

		static inline void OperatorTanH(
			float* RESTRICT Output,
			const float* RESTRICT Input,
			const uint32 BatchSize,
			const uint32 InputOutputSize,
			const uint32 OutputStride,
			const uint32 InputStride)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::OperatorTanH);

			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZE(Output, Input, InputOutputSize);

#if NNE_RUNTIME_BASIC_ENABLE_ISPC
			ispc::NNERuntimeBasicCPUOperatorTanH(
				Output,
				Input,
				BatchSize,
				InputOutputSize,
				OutputStride,
				InputStride);
#else
			for (uint32 BatchIdx = 0; BatchIdx < BatchSize; BatchIdx++)
			{
				for (uint32 Idx = 0; Idx < InputOutputSize; Idx++)
				{
					Output[BatchIdx * OutputStride + Idx] = FMath::Tanh(Input[BatchIdx * InputStride + Idx]);
				}
			}
#endif
		}

		static inline void OperatorPReLU(
			float* RESTRICT Output,
			const float* RESTRICT Input,
			const float* RESTRICT Alpha,
			const uint32 BatchSize,
			const uint32 InputOutputSize,
			const uint32 OutputStride,
			const uint32 InputStride)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::OperatorPReLU);

			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZE(Output, Input, InputOutputSize);

#if NNE_RUNTIME_BASIC_ENABLE_ISPC
			ispc::NNERuntimeBasicCPUOperatorPReLU(
				Output,
				Input,
				Alpha,
				BatchSize,
				InputOutputSize,
				OutputStride,
				InputStride);
#else
			for (uint32 BatchIdx = 0; BatchIdx < BatchSize; BatchIdx++)
			{
				for (uint32 Idx = 0; Idx < InputOutputSize; Idx++)
				{
					const float Value = Input[BatchIdx * InputStride + Idx];
					Output[BatchIdx * OutputStride + Idx] = Value > 0.0f ? Value : Alpha[Idx] * Value;
				}
			}
#endif
		}

		static inline void OperatorSoftmax(
			float* RESTRICT Output,
			const float* RESTRICT Input,
			const uint32 BatchSize,
			const uint32 InputOutputSize,
			const uint32 OutputStride,
			const uint32 InputStride)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::OperatorSoftmax);

			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZE(Output, Input, InputOutputSize);

#if NNE_RUNTIME_BASIC_ENABLE_ISPC
			ispc::NNERuntimeBasicCPUOperatorSoftmax(
				Output,
				Input,
				BatchSize,
				InputOutputSize,
				OutputStride,
				InputStride);
#else
			for (uint32 BatchIdx = 0; BatchIdx < BatchSize; BatchIdx++)
			{
				float Max = 0.0f;
				for (uint32 Idx = 0; Idx < InputOutputSize; Idx++)
				{
					Max = FMath::Max(Max, Input[BatchIdx * InputStride + Idx]);
				}

				float Denom = 0.0f;
				for (uint32 Idx = 0; Idx < InputOutputSize; Idx++)
				{
					Denom += FMath::Exp(Input[BatchIdx * InputStride + Idx] - Max);
				}
				Denom = FMath::Max(Denom, 1e-8f);

				for (uint32 Idx = 0; Idx < InputOutputSize; Idx++)
				{
					Output[BatchIdx * OutputStride + Idx] = FMath::Exp(Input[BatchIdx * InputStride + Idx] - Max) / Denom;
				}
			}
#endif
		}

		static inline void OperatorMemoryCellUpdateMemory(
			float* RESTRICT Output,
			const float* RESTRICT RememberGate,
			const float* RESTRICT Memory,
			const float* RESTRICT Update,
			const uint32 BatchSize,
			const uint32 MemorySize,
			const uint32 OutputStride,
			const uint32 RememberGateStride,
			const uint32 MemoryStride,
			const uint32 UpdateStride)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::OperatorMemoryCellUpdateMemory);

			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZE(Output, RememberGate, MemorySize);
			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZE(Output, Memory, MemorySize);
			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZE(Output, Update, MemorySize);

#if NNE_RUNTIME_BASIC_ENABLE_ISPC
			ispc::NNERuntimeBasicCPUOperatorMemoryCellUpdateMemory(
				Output,
				RememberGate,
				Memory,
				Update,
				BatchSize,
				MemorySize,
				OutputStride,
				RememberGateStride,
				MemoryStride,
				UpdateStride);
#else
			for (uint32 BatchIdx = 0; BatchIdx < BatchSize; BatchIdx++)
			{
				for (uint32 Idx = 0; Idx < MemorySize; Idx++)
				{
					const float Gate = Sigmoid(RememberGate[BatchIdx * RememberGateStride + Idx]);
					const float Prev = Memory[BatchIdx * MemoryStride + Idx];
					const float Targ = FMath::Tanh(Update[BatchIdx * UpdateStride + Idx]);

					Output[BatchIdx * OutputStride + Idx] = (1.0f - Gate) * Prev + Gate * Targ;
				}
			}
#endif
		}

		static inline void OperatorMemoryCellUpdateOutput(
			float* RESTRICT Output,
			const float* RESTRICT PassthroughGate,
			const float* RESTRICT MemoryUpdate,
			const float* RESTRICT InputUpdate,
			const uint32 BatchSize,
			const uint32 OutputSize,
			const uint32 OutputStride,
			const uint32 PassthroughGateStride,
			const uint32 MemoryUpdateStride,
			const uint32 InputUpdateStride)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::OperatorMemoryCellUpdateOutput);

			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZE(Output, PassthroughGate, OutputSize);
			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZE(Output, MemoryUpdate, OutputSize);
			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZE(Output, InputUpdate, OutputSize);

#if NNE_RUNTIME_BASIC_ENABLE_ISPC
			ispc::NNERuntimeBasicCPUOperatorMemoryCellUpdateOutput(
				Output,
				PassthroughGate,
				MemoryUpdate,
				InputUpdate,
				BatchSize,
				OutputSize,
				OutputStride,
				PassthroughGateStride,
				MemoryUpdateStride,
				InputUpdateStride);
#else
			for (uint32 BatchIdx = 0; BatchIdx < BatchSize; BatchIdx++)
			{
				for (uint32 Idx = 0; Idx < OutputSize; Idx++)
				{
					const float Gate = Sigmoid(PassthroughGate[BatchIdx * PassthroughGateStride + Idx]);
					const float MemTarg = FMath::Tanh(MemoryUpdate[BatchIdx * MemoryUpdateStride + Idx]);
					const float InTarg = FMath::Tanh(InputUpdate[BatchIdx * InputUpdateStride + Idx]);

					Output[BatchIdx * OutputStride + Idx] = (1.0f - Gate) * MemTarg + Gate * InTarg;
				}
			}
#endif
		}

		static inline void OperatorGRUMaskInput(
			float* RESTRICT Masked,
			const float* RESTRICT InputMemory,
			const float* RESTRICT RememberGate,
			const uint32 BatchSize,
			const uint32 OutputSize,
			const uint32 InputSize,
			const uint32 MaskedStride,
			const uint32 InputMemoryStride,
			const uint32 RememberGateStride)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::OperatorGRUMaskInput);

			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZE(Masked, InputMemory, InputSize + OutputSize);
			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZES(Masked, RememberGate, InputSize + OutputSize, OutputSize);

#if NNE_RUNTIME_BASIC_ENABLE_ISPC
			ispc::NNERuntimeBasicCPUOperatorGRUMaskInput(
				Masked,
				InputMemory,
				RememberGate,
				BatchSize,
				OutputSize,
				InputSize,
				MaskedStride,
				InputMemoryStride,
				RememberGateStride);
#else
			for (uint32 BatchIdx = 0; BatchIdx < BatchSize; BatchIdx++)
			{
				for (uint32 Idx = 0; Idx < InputSize; Idx++)
				{
					Masked[BatchIdx * MaskedStride + Idx] = InputMemory[BatchIdx * InputMemoryStride + Idx];
				}

				for (uint32 Idx = 0; Idx < OutputSize; Idx++)
				{
					Masked[BatchIdx * MaskedStride + InputSize + Idx] = 
						Sigmoid(RememberGate[BatchIdx * RememberGateStride + Idx]) * 
						InputMemory[BatchIdx * InputMemoryStride + InputSize + Idx];
				}
			}
#endif

		}


		static inline void OperatorGRUCellUpdateOutput(
			float* RESTRICT OutputMemory,
			const float* RESTRICT InputMemory,
			const float* RESTRICT UpdateGate ,
			const float* RESTRICT ActivationGate,
			const uint32 BatchSize,
			const uint32 OutputSize,
			const uint32 InputSize,
			const uint32 OutputMemoryStride,
			const uint32 InputMemoryStride,
			const uint32 UpdateGateStride,
			const uint32 ActivationGateStride)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::OperatorGRUCellUpdateOutput);

			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZES(OutputMemory, InputMemory, OutputSize + OutputSize, InputSize + OutputSize);
			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZES(OutputMemory, UpdateGate, OutputSize + OutputSize, OutputSize);
			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZES(OutputMemory, ActivationGate, OutputSize + OutputSize, OutputSize);

#if NNE_RUNTIME_BASIC_ENABLE_ISPC
			ispc::NNERuntimeBasicCPUOperatorGRUCellUpdateOutput(
				OutputMemory,
				InputMemory,
				UpdateGate,
				ActivationGate,
				BatchSize,
				OutputSize,
				InputSize,
				OutputMemoryStride,
				InputMemoryStride,
				UpdateGateStride,
				ActivationGateStride);
#else
			for (uint32 BatchIdx = 0; BatchIdx < BatchSize; BatchIdx++)
			{
				for (uint32 Idx = 0; Idx < OutputSize; Idx++)
				{
					const float MemoryValue = InputMemory[BatchIdx * InputMemoryStride + InputSize + Idx];
					const float UpdateGateValue = Sigmoid(UpdateGate[BatchIdx * UpdateGateStride + Idx]);
					const float ActivationGateValue = FMath::Tanh(ActivationGate[BatchIdx * ActivationGateStride + Idx]);
					const float OutputValue = (1.0f - UpdateGateValue) * MemoryValue + UpdateGateValue * ActivationGateValue;

					OutputMemory[BatchIdx * OutputMemoryStride + Idx] = OutputValue;
					OutputMemory[BatchIdx * OutputMemoryStride + OutputSize + Idx] = OutputValue;
				}
			}
#endif
		}

		static inline void OperatorAggregateGatherElements(
			float* RESTRICT OutputBuffer,
			const float* RESTRICT InputBuffer,
			const uint32* RESTRICT ElementNums,
			const uint32* RESTRICT ElementOffsets,
			const uint32 BatchSize,
			const uint32 ElementSize,
			const uint32 InputStride)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::OperatorAggregateGatherElements);

			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZE(OutputBuffer, InputBuffer, ElementSize);

#if NNE_RUNTIME_BASIC_ENABLE_ISPC
			ispc::NNERuntimeBasicCPUOperatorAggregateGatherElements(
				OutputBuffer,
				InputBuffer,
				ElementNums,
				ElementOffsets,
				BatchSize,
				ElementSize,
				InputStride);
#else
			for (uint32 BatchIdx = 0; BatchIdx < BatchSize; BatchIdx++)
			{
				const uint32 ElementNum = ElementNums[BatchIdx];
				const uint32 ElementOffset = ElementOffsets[BatchIdx];

				for (uint32 ElementIdx = 0; ElementIdx < ElementNum; ElementIdx++)
				{
					for (uint32 Idx = 0; Idx < ElementSize; Idx++)
					{
						OutputBuffer[(ElementOffset + ElementIdx) * ElementSize + Idx] = InputBuffer[BatchIdx * InputStride + ElementIdx * ElementSize + Idx];
					}
				}
			}
#endif
		}

		static inline void OperatorAggregateInsertOneHot(
			float* RESTRICT QueryBuffer,
			const uint32 Index,
			const uint32 BatchSize,
			const uint32 MaskSize,
			const uint32 QueryBufferStride)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::OperatorAggregateInsertOneHot);

#if NNE_RUNTIME_BASIC_ENABLE_ISPC
			ispc::NNERuntimeBasicCPUOperatorAggregateInsertOneHot(
				QueryBuffer,
				Index,
				BatchSize,
				MaskSize,
				QueryBufferStride);
#else
			for (uint32 BatchIdx = 0; BatchIdx < BatchSize; BatchIdx++)
			{
				for (uint32 MaskIdx = 0; MaskIdx < MaskSize; MaskIdx++)
				{
					QueryBuffer[BatchIdx * QueryBufferStride + MaskIdx] = 0.0f;
				}

				QueryBuffer[BatchIdx * QueryBufferStride + Index] = 1.0f;
			}
#endif
		}

		static inline void OperatorAggregateCountElementNum(
			uint32& TotalElementNum,
			uint32* RESTRICT ElementNums,
			uint32* RESTRICT ElementOffsets,
			const float* RESTRICT MaskBuffer,
			const uint32 BatchSize,
			const uint32 MaskSize,
			const uint32 MaskBufferStride)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::OperatorAggregateCountElementNum);

#if NNE_RUNTIME_BASIC_ENABLE_ISPC
			ispc::NNERuntimeBasicCPUOperatorAggregateCountElementNum(
				TotalElementNum,
				ElementNums,
				ElementOffsets,
				MaskBuffer,
				BatchSize,
				MaskSize,
				MaskBufferStride);
#else
			TotalElementNum = 0;

			for (uint32 BatchIdx = 0; BatchIdx < BatchSize; BatchIdx++)
			{
				uint32 ElementSum = 0;
				for (uint32 MaskIdx = 0; MaskIdx < MaskSize; MaskIdx++)
				{
					if (MaskBuffer[BatchIdx * MaskBufferStride + MaskIdx]) { ElementSum++; }
				}

				ElementOffsets[BatchIdx] = TotalElementNum;
				ElementNums[BatchIdx] = ElementSum;
				TotalElementNum += ElementSum;
			}
#endif
		}

		static inline void OperatorAggregateGatherFromSubLayers(
			float* RESTRICT QueryBuffer,
			float* RESTRICT KeyBuffer,
			float* RESTRICT ValueBuffer,
			uint32* RESTRICT ElementAccum,
			const uint32* RESTRICT ElementNums,
			const uint32* RESTRICT ElementOffsets,
			const TConstArrayView<TArray<uint32>> SubLayerBatchIndices,
			const TConstArrayView<TArray<float>> SubLayerQueryBuffers,
			const TConstArrayView<TArray<float>> SubLayerKeyBuffers,
			const TConstArrayView<TArray<float>> SubLayerValueBuffers,
			const uint32 BatchSize,
			const uint32 QuerySize,
			const uint32 KeySize,
			const uint32 ValueSize)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::OperatorAggregateGatherFromSubLayers);

			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZES(QueryBuffer, KeyBuffer, QuerySize, KeySize);
			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZES(QueryBuffer, ValueBuffer, QuerySize, ValueSize);
			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZES(KeyBuffer, ValueBuffer, KeySize, ValueSize);
			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZE(ElementAccum, ElementNums, BatchSize);
			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZE(ElementAccum, ElementOffsets, BatchSize);

			const uint32 SubLayerNum = SubLayerBatchIndices.Num();

			for (uint32 BatchIdx = 0; BatchIdx < BatchSize; BatchIdx++)
			{
				ElementAccum[BatchIdx] = 0;
			}

			for (uint32 SubLayerIdx = 0; SubLayerIdx < SubLayerNum; SubLayerIdx++)
			{
				const float* RESTRICT SubLayerQueryBuffer = SubLayerQueryBuffers[SubLayerIdx].GetData();
				const float* RESTRICT SubLayerKeyBuffer = SubLayerKeyBuffers[SubLayerIdx].GetData();
				const float* RESTRICT SubLayerValueBuffer = SubLayerValueBuffers[SubLayerIdx].GetData();
				const uint32* RESTRICT SubLayerBatchIndicesBuffer = SubLayerBatchIndices[SubLayerIdx].GetData();
				const uint32 SubLayerBatchIndexNum = SubLayerBatchIndices[SubLayerIdx].Num();

#if NNE_RUNTIME_BASIC_ENABLE_ISPC
				ispc::NNERuntimeBasicCPUOperatorAggregateGatherQueryValueFromSubLayers(
					QueryBuffer,
					KeyBuffer,
					ValueBuffer,
					ElementAccum,
					ElementOffsets,
					SubLayerQueryBuffer,
					SubLayerKeyBuffer,
					SubLayerValueBuffer,
					SubLayerBatchIndicesBuffer,
					SubLayerBatchIndexNum,
					QuerySize,
					KeySize,
					ValueSize);
#else
				for (uint32 ElementIdx = 0; ElementIdx < SubLayerBatchIndexNum; ElementIdx++)
				{
					const uint32 BatchIdx = SubLayerBatchIndicesBuffer[ElementIdx];
					const uint32 ElementOffset = ElementOffsets[BatchIdx] + ElementAccum[BatchIdx];

					for (uint32 QueryIdx = 0; QueryIdx < QuerySize; QueryIdx++)
					{
						QueryBuffer[ElementOffset * QuerySize + QueryIdx] = SubLayerQueryBuffer[ElementIdx * QuerySize + QueryIdx];
					}

					for (uint32 KeyIdx = 0; KeyIdx < KeySize; KeyIdx++)
					{
						KeyBuffer[ElementOffset * KeySize + KeyIdx] = SubLayerKeyBuffer[ElementIdx * KeySize + KeyIdx];
					}

					for (uint32 ValueIdx = 0; ValueIdx < ValueSize; ValueIdx++)
					{
						ValueBuffer[ElementOffset * ValueSize + ValueIdx] = SubLayerValueBuffer[ElementIdx * ValueSize + ValueIdx];
					}

					ElementAccum[BatchIdx]++;
				}
#endif
			}

			for (uint32 BatchIdx = 0; BatchIdx < BatchSize; BatchIdx++)
			{
				check(ElementAccum[BatchIdx] == ElementNums[BatchIdx]);
			}
		}

		static inline void OperatorAggregateDotProductAttention(
			float* RESTRICT Attention,
			const float* RESTRICT Queries,
			const float* RESTRICT Keys,
			const uint32 ElementNum,
			const uint32 AttentionEncodingSize,
			const uint32 AttentionHeadNum)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::OperatorAggregateDotProductAttention);

			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZE(Attention, Queries, AttentionEncodingSize);
			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZE(Attention, Keys, AttentionEncodingSize);

#if NNE_RUNTIME_BASIC_ENABLE_ISPC
			ispc::NNERuntimeBasicCPUOperatorAggregateDotProductAttention(
				Attention,
				Queries,
				Keys,
				ElementNum,
				AttentionEncodingSize,
				AttentionHeadNum);
#else
			for (uint32 ElementIdx = 0; ElementIdx < ElementNum; ElementIdx++)
			{
				for (uint32 HeadIdx = 0; HeadIdx < AttentionHeadNum; HeadIdx++)
				{
					Attention[ElementIdx * AttentionHeadNum + HeadIdx] = 0.0f;

					for (uint32 Idx = 0; Idx < AttentionEncodingSize; Idx++)
					{
						Attention[ElementIdx * AttentionHeadNum + HeadIdx] += (
							Keys[ElementIdx * AttentionHeadNum * AttentionEncodingSize + HeadIdx * AttentionEncodingSize + Idx] *
							Queries[ElementIdx * AttentionHeadNum * AttentionEncodingSize + HeadIdx * AttentionEncodingSize + Idx]);
					}

					Attention[ElementIdx * AttentionHeadNum + HeadIdx] /= FMath::Sqrt((float)AttentionEncodingSize);
				}
			}
#endif
		}

		static inline void OperatorEncodeElementNums(
			float* RESTRICT OutputBuffer,
			const uint32* RESTRICT ElementNums,
			const uint32 MaxElementNum,
			const uint32 BatchSize,
			const uint32 OutputStride)
		{
			for (uint32 BatchIdx = 0; BatchIdx < BatchSize; BatchIdx++)
			{
				OutputBuffer[BatchIdx * OutputStride] = (float)ElementNums[BatchIdx] / (float)MaxElementNum;
			}
		}

		static inline void OperatorAggregateSoftmaxPlusOneInplace(
			float* RESTRICT AttentionMaxs,
			float* RESTRICT AttentionDenoms,
			float* RESTRICT Attention,
			const uint32* RESTRICT ElementNums,
			const uint32* RESTRICT ElementOffsets,
			const uint32 BatchSize,
			const uint32 AttentionHeadNum)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::OperatorAggregateSoftmaxPlusOneInplace);

			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZE(AttentionMaxs, AttentionDenoms, AttentionHeadNum);
			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZE(AttentionMaxs, Attention, AttentionHeadNum);
			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZE(Attention, AttentionDenoms, AttentionHeadNum);

			// Numerically stable soft-max computation using subtraction of the (positive) max value
			// 
			// Here the +1 in the denominator allows the attention 
			// to attend to nothing as discussed here:
			// https://www.evanmiller.org/attention-is-off-by-one.html

#if NNE_RUNTIME_BASIC_ENABLE_ISPC
			ispc::NNERuntimeBasicCPUOperatorAggregateSoftmaxPlusOneInplace(
				AttentionMaxs,
				AttentionDenoms,
				Attention,
				ElementNums,
				ElementOffsets,
				BatchSize,
				AttentionHeadNum);
#else
			for (uint32 BatchIdx = 0; BatchIdx < BatchSize; BatchIdx++)
			{
				const uint32 ElementNum = ElementNums[BatchIdx];
				const uint32 ElementOffset = ElementOffsets[BatchIdx];

				for (uint32 HeadIdx = 0; HeadIdx < AttentionHeadNum; HeadIdx++)
				{
					AttentionMaxs[HeadIdx] = 0.0f;
					AttentionDenoms[HeadIdx] = 0.0f;
				}

				for (uint32 ElementIdx = ElementOffset; ElementIdx < ElementOffset + ElementNum; ElementIdx++)
				{
					for (uint32 HeadIdx = 0; HeadIdx < AttentionHeadNum; HeadIdx++)
					{
						AttentionMaxs[HeadIdx] = FMath::Max(AttentionMaxs[HeadIdx], Attention[ElementIdx * AttentionHeadNum + HeadIdx]);
					}
				}

				for (uint32 ElementIdx = ElementOffset; ElementIdx < ElementOffset + ElementNum; ElementIdx++)
				{
					for (uint32 HeadIdx = 0; HeadIdx < AttentionHeadNum; HeadIdx++)
					{
						AttentionDenoms[HeadIdx] += FMath::Exp(Attention[ElementIdx * AttentionHeadNum + HeadIdx] - AttentionMaxs[HeadIdx]);
					}
				}

				for (uint32 ElementIdx = ElementOffset; ElementIdx < ElementOffset + ElementNum; ElementIdx++)
				{
					for (uint32 HeadIdx = 0; HeadIdx < AttentionHeadNum; HeadIdx++)
					{
						Attention[ElementIdx * AttentionHeadNum + HeadIdx] = FMath::Exp(Attention[ElementIdx * AttentionHeadNum + HeadIdx] - AttentionMaxs[HeadIdx]) / (AttentionDenoms[HeadIdx] + 1.0f);
					}
				}
			}
#endif
		}

		static inline void OperatorAggregateAttentionSum(
			float* RESTRICT Output,
			const float* RESTRICT Attention,
			const float* RESTRICT Values,
			const uint32* RESTRICT ElementNums,
			const uint32* RESTRICT ElementOffsets,
			const uint32 BatchSize,
			const uint32 EncodingSize,
			const uint32 AttentionHeadNum,
			const uint32 OutputStride)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::OperatorAggregateAttentionSum);

			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZES(Output, Attention, EncodingSize, AttentionHeadNum);
			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZE(Output, Values, EncodingSize);

#if NNE_RUNTIME_BASIC_ENABLE_ISPC
			ispc::NNERuntimeBasicCPUOperatorAggregateAttentionSum(
				Output,
				Attention,
				Values,
				ElementNums,
				ElementOffsets,
				BatchSize,
				EncodingSize,
				AttentionHeadNum,
				OutputStride);
#else
			for (uint32 BatchIdx = 0; BatchIdx < BatchSize; BatchIdx++)
			{
				for (uint32 HeadIdx = 0; HeadIdx < AttentionHeadNum; HeadIdx++)
				{
					for (uint32 Idx = 0; Idx < EncodingSize; Idx++)
					{
						Output[BatchIdx * OutputStride + HeadIdx * EncodingSize + Idx] = 0.0f;
					}
				}

				const uint32 ElementNum = ElementNums[BatchIdx];
				const uint32 ElementOffset = ElementOffsets[BatchIdx];

				for (uint32 ElementIdx = ElementOffset; ElementIdx < ElementOffset + ElementNum; ElementIdx++)
				{
					for (uint32 HeadIdx = 0; HeadIdx < AttentionHeadNum; HeadIdx++)
					{
						const float Scale = Attention[ElementIdx * AttentionHeadNum + HeadIdx];

						if (Scale != 0.0f)
						{
							for (uint32 Idx = 0; Idx < EncodingSize; Idx++)
							{
								Output[BatchIdx * OutputStride + HeadIdx * EncodingSize + Idx] += Scale * Values[ElementIdx * AttentionHeadNum * EncodingSize + HeadIdx * EncodingSize + Idx];
							}
						}
					}
				}
			}
#endif
		}

		static inline void OperatorGather(
			float* RESTRICT OutputBuffer,
			const float* RESTRICT InputBuffer,
			const uint32* RESTRICT BatchIndices,
			const uint32 BatchIndexNum,
			const uint32 InputOutputSize,
			const uint32 OutputStride,
			const uint32 InputStride)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::OperatorGather);

			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZE(OutputBuffer, InputBuffer, InputOutputSize);

#if NNE_RUNTIME_BASIC_ENABLE_ISPC
			ispc::NNERuntimeBasicCPUOperatorGather(
				OutputBuffer,
				InputBuffer,
				BatchIndices,
				BatchIndexNum,
				InputOutputSize,
				OutputStride,
				InputStride);
#else
			for (uint32 BatchIndexIdx = 0; BatchIndexIdx < BatchIndexNum; BatchIndexIdx++)
			{
				const uint32 SrcBatchIdx = BatchIndices[BatchIndexIdx];
				for (uint32 Idx = 0; Idx < InputOutputSize; Idx++)
				{
					OutputBuffer[BatchIndexIdx * OutputStride + Idx] = InputBuffer[SrcBatchIdx * InputStride + Idx];
				}
			}
#endif
		}

		static inline void OperatorScatter(
			float* RESTRICT OutputBuffer,
			const float* RESTRICT InputBuffer,
			const uint32* RESTRICT BatchIndices,
			const uint32 BatchIndexNum,
			const uint32 InputOutputSize,
			const uint32 OutputStride,
			const uint32 InputStride)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::OperatorScatter);

			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZE(OutputBuffer, InputBuffer, InputOutputSize);

#if NNE_RUNTIME_BASIC_ENABLE_ISPC
			ispc::NNERuntimeBasicCPUOperatorScatter(
				OutputBuffer,
				InputBuffer,
				BatchIndices,
				BatchIndexNum,
				InputOutputSize,
				OutputStride,
				InputStride);
#else
			for (uint32 BatchIndexIdx = 0; BatchIndexIdx < BatchIndexNum; BatchIndexIdx++)
			{
				const uint32 DstBatchIdx = BatchIndices[BatchIndexIdx];
				for (uint32 Idx = 0; Idx < InputOutputSize; Idx++)
				{
					OutputBuffer[DstBatchIdx * OutputStride + Idx] = InputBuffer[BatchIndexIdx * InputStride + Idx];
				}
			}
#endif
		}

		static inline void OperatorGatherSubLayerBatchIndicesExclusive(
			TArrayView<TArray<uint32>> SubLayerBatchIndices,
			const float* RESTRICT SubLayerMaskBuffer,
			const uint32 BatchSize,
			const uint32 SubLayerMaskSize,
			const uint32 SubLayerMaskStride)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::OperatorGatherSubLayerBatchIndicesExclusive);

			for (uint32 SubLayerIdx = 0; SubLayerIdx < SubLayerMaskSize; SubLayerIdx++)
			{
				SubLayerBatchIndices[SubLayerIdx].Reset();
			}

			for (uint32 BatchIdx = 0; BatchIdx < BatchSize; BatchIdx++)
			{
				bool bFound = false;
				for (uint32 SubLayerIdx = 0; SubLayerIdx < SubLayerMaskSize; SubLayerIdx++)
				{
					if (SubLayerMaskBuffer[BatchIdx * SubLayerMaskStride + SubLayerIdx])
					{
						SubLayerBatchIndices[SubLayerIdx].Add(BatchIdx);
						bFound = true;
						break;
					}
				}

				checkf(bFound, TEXT("SubLayer index not found."));
			}
		}

		static inline void OperatorGatherSubLayerBatchIndicesInclusive(
			TArrayView<TArray<uint32>> SubLayerBatchIndices,
			const float* RESTRICT SubLayerMaskBuffer,
			const uint32 BatchSize,
			const uint32 SubLayerMaskSize,
			const uint32 SubLayerMaskStride)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::OperatorGatherSubLayerBatchIndicesInclusive);

			for (uint32 SubLayerIdx = 0; SubLayerIdx < SubLayerMaskSize; SubLayerIdx++)
			{
				SubLayerBatchIndices[SubLayerIdx].Reset();
			}

			for (uint32 BatchIdx = 0; BatchIdx < BatchSize; BatchIdx++)
			{
				for (uint32 SubLayerIdx = 0; SubLayerIdx < SubLayerMaskSize; SubLayerIdx++)
				{
					if (SubLayerMaskBuffer[BatchIdx * SubLayerMaskStride + SubLayerIdx])
					{
						SubLayerBatchIndices[SubLayerIdx].Add(BatchIdx);
					}
				}
			}
		}

		static inline void OperatorGatherTopTwoSubLayerBatchIndices(
			TArrayView<TArray<uint32>> SubLayerBatchIndices,
			uint32* RESTRICT BatchSubLayerIndex0,
			uint32* RESTRICT BatchSubLayerIndex1,
			float* RESTRICT BatchSubLayerWeight0,
			float* RESTRICT BatchSubLayerWeight1,
			uint32* RESTRICT BatchSubLayerOutputIndex0,
			uint32* RESTRICT BatchSubLayerOutputIndex1,
			const float* RESTRICT SubLayerGateBuffer,
			const uint32 BatchSize,
			const uint32 SubLayerGateSize,
			const uint32 SubLayerGateStride)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::OperatorGatherTopTwoSubLayerBatchIndices);

			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZE(BatchSubLayerIndex0, BatchSubLayerIndex1, BatchSize);
			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZE(BatchSubLayerWeight0, BatchSubLayerWeight1, BatchSize);
			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZE(BatchSubLayerOutputIndex0, BatchSubLayerOutputIndex1, BatchSize);

			check(SubLayerGateSize >= 2);

			for (uint32 SubLayerIdx = 0; SubLayerIdx < SubLayerGateSize; SubLayerIdx++)
			{
				SubLayerBatchIndices[SubLayerIdx].Reset();
			}

			for (uint32 BatchIdx = 0; BatchIdx < BatchSize; BatchIdx++)
			{
				int32 BestIdx0 = INDEX_NONE;
				int32 BestIdx1 = INDEX_NONE;
				float BestVal0 = -FLT_MAX;
				float BestVal1 = -FLT_MAX;

				for (uint32 SubLayerIdx = 0; SubLayerIdx < SubLayerGateSize; SubLayerIdx++)
				{
					const float GateValue = SubLayerGateBuffer[BatchIdx * SubLayerGateStride + SubLayerIdx];

					if (GateValue > BestVal0)
					{
						BestIdx1 = BestIdx0;
						BestVal1 = BestVal0;
						BestIdx0 = SubLayerIdx;
						BestVal0 = GateValue;
						continue;
					}

					if (GateValue > BestVal1)
					{
						BestIdx1 = SubLayerIdx;
						BestVal1 = GateValue;
						continue;
					}
				}

				const float ExpVal0 = FMath::Exp(BestVal0 - FMath::Max(BestVal0, BestVal1));
				const float ExpVal1 = FMath::Exp(BestVal1 - FMath::Max(BestVal0, BestVal1));

				BatchSubLayerIndex0[BatchIdx] = BestIdx0;
				BatchSubLayerIndex1[BatchIdx] = BestIdx1;
				BatchSubLayerWeight0[BatchIdx] = ExpVal0 / (ExpVal0 + ExpVal1);
				BatchSubLayerWeight1[BatchIdx] = ExpVal1 / (ExpVal0 + ExpVal1);
				BatchSubLayerOutputIndex0[BatchIdx] = SubLayerBatchIndices[BestIdx0].Add(BatchIdx);
				BatchSubLayerOutputIndex1[BatchIdx] = SubLayerBatchIndices[BestIdx1].Add(BatchIdx);
			}
		}

		static inline void OperatorGatherTopTwoFromSubLayers(
			float* RESTRICT OutputBuffer,
			const uint32* RESTRICT BatchSubLayerIndex0,
			const uint32* RESTRICT BatchSubLayerIndex1,
			const float* RESTRICT BatchSubLayerWeight0,
			const float* RESTRICT BatchSubLayerWeight1,
			const uint32* RESTRICT BatchSubLayerOutputIndex0,
			const uint32* RESTRICT BatchSubLayerOutputIndex1,
			const TConstArrayView<TArray<float>> SubLayerOutputBuffer,
			const uint32 BatchSize,
			const uint32 OutputBufferSize,
			const uint32 SubLayerOutputStride,
			const uint32 OutputBufferStride)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::OperatorGatherTopTwoFromSubLayers);

			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZES(OutputBuffer, BatchSubLayerWeight0, OutputBufferSize, BatchSize);
			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZES(OutputBuffer, BatchSubLayerWeight1, OutputBufferSize, BatchSize);

			for (uint32 BatchIdx = 0; BatchIdx < BatchSize; BatchIdx++)
			{
				const uint32 SubLayerIndex0 = BatchSubLayerIndex0[BatchIdx];
				const uint32 SubLayerIndex1 = BatchSubLayerIndex1[BatchIdx];
				const float Weight0 = BatchSubLayerWeight0[BatchIdx];
				const float Weight1 = BatchSubLayerWeight1[BatchIdx];
				const uint32 SubLayerOutputIndex0 = BatchSubLayerOutputIndex0[BatchIdx];
				const uint32 SubLayerOutputIndex1 = BatchSubLayerOutputIndex1[BatchIdx];

				const float* RESTRICT SubLayerOutputBuffer0 = SubLayerOutputBuffer[SubLayerIndex0].GetData();
				const float* RESTRICT SubLayerOutputBuffer1 = SubLayerOutputBuffer[SubLayerIndex1].GetData();

				NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZE(OutputBuffer, SubLayerOutputBuffer0, OutputBufferSize);
				NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZE(OutputBuffer, SubLayerOutputBuffer1, OutputBufferSize);

				for (uint32 OutputIdx = 0; OutputIdx < OutputBufferSize; OutputIdx++)
				{
					OutputBuffer[BatchIdx * OutputBufferStride + OutputIdx] =
						Weight0 * SubLayerOutputBuffer0[SubLayerOutputIndex0 * SubLayerOutputStride + OutputIdx] +
						Weight1 * SubLayerOutputBuffer1[SubLayerOutputIndex1 * SubLayerOutputStride + OutputIdx];
				}
			}
		}

		static inline void OperatorLayerNorm(
			float* RESTRICT Output,
			const float* RESTRICT Input,
			const float* RESTRICT Offset,
			const float* RESTRICT Scale,
			const float Epsilon,
			const uint32 BatchSize,
			const uint32 InputOutputSize,
			const uint32 OutputStride,
			const uint32 InputStride)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::OperatorLayerNorm);

			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZE(Output, Input, InputOutputSize);

#if NNE_RUNTIME_BASIC_ENABLE_ISPC
			ispc::NNERuntimeBasicCPUOperatorLayerNorm(
				Output,
				Input,
				Offset,
				Scale,
				Epsilon,
				BatchSize,
				InputOutputSize,
				OutputStride,
				InputStride);
#else
			for (uint32 BatchIdx = 0; BatchIdx < BatchSize; BatchIdx++)
			{
				float Mean = 0.0f;
				for (uint32 Idx = 0; Idx < InputOutputSize; Idx++)
				{
					Mean += Input[BatchIdx * InputStride + Idx] / InputOutputSize;
				}

				float Std = 0.0f;
				for (uint32 Idx = 0; Idx < InputOutputSize; Idx++)
				{
					Std += FMath::Square(Input[BatchIdx * InputStride + Idx] - Mean) / InputOutputSize;
				}
				Std = FMath::Sqrt(Std + Epsilon);

				for (uint32 Idx = 0; Idx < InputOutputSize; Idx++)
				{
					Output[BatchIdx * OutputStride + Idx] = ((Input[BatchIdx * InputStride + Idx] - Mean) / Std) * Scale[Idx] + Offset[Idx];
				}
			}
#endif
		}

		static inline void OperatorLayerFiLM(
			float* RESTRICT InputOutput,
			const float* RESTRICT Condition,
			const uint32 BatchSize,
			const uint32 InputOutputSize,
			const uint32 InputOutputStride,
			const uint32 ConditionStride)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::OperatorLayerFiLM);

			NNE_RUNTIME_BASIC_CHECK_ALIASING_SIZE(InputOutput, Condition, InputOutputSize);

#if NNE_RUNTIME_BASIC_ENABLE_ISPC
			ispc::NNERuntimeBasicCPUOperatorLayerFiLM(
				InputOutput,
				Condition,
				BatchSize,
				InputOutputSize,
				InputOutputStride,
				ConditionStride);
#else
			for (uint32 BatchIdx = 0; BatchIdx < BatchSize; BatchIdx++)
			{
				for (uint32 Idx = 0; Idx < InputOutputSize; Idx++)
				{
					InputOutput[BatchIdx * InputOutputStride + Idx] =
						(InputOutput[BatchIdx * InputOutputStride + Idx] * 
						Condition[BatchIdx * ConditionStride + Idx]) + 
						Condition[BatchIdx * ConditionStride + InputOutputSize + Idx];
				}
			}
#endif
		}

		//--------------------------------------------------------------------------
		// Layer Types
		//--------------------------------------------------------------------------

		/** Layer Type Id - this should match what is given in nne_runtime_basic_cpu.py  */
		enum class ELayerType : uint32
		{
			Invalid = 0,
			Sequence = 1,
			Normalize = 2,
			Denormalize = 3,
			Linear = 4,
			CompressedLinear = 5,
			MultiLinear = 6,
			ReLU = 7,
			ELU = 8,
			TanH = 9,
			PReLU = 10,
			MemoryCell = 11,
			Copy = 12,
			Concat = 13,
			Array = 14,
			AggregateSet = 15,
			AggregateOrExclusive = 16,
			AggregateOrInclusive = 17,
			Clamp = 18,
			SparseMixtureOfExperts = 19,
			GELU = 20,
			LayerNorm = 21,
			LipschiztLinear = 22,
			Tile = 23,
			Spread = 24,
			Slice = 25,
			Residual = 26,
			FiLM = 27,
			GRUCell = 28,
			Conv1d = 29,
			Conv2d = 30,
			Softmax = 31,
		};

		//--------------------------------------------------------------------------
		// Layer Type Interfaces
		//--------------------------------------------------------------------------

		/**
		 * Interface for a Layer Instance - the data required for performing inference for a layer.
		 */
		struct ILayerInstance
		{
			/** Virtual destructor */
			virtual ~ILayerInstance() = default;

			/** Indicate to this layer instance what the maximum batchsize is going to be when performing inference. */
			virtual void SetMaxBatchSize(const uint32 MaxBatchSize) = 0;

			/** Gets the type of layer this is an instance for */
			virtual ELayerType GetLayerType() const = 0;
		};

		/**
		 * Interface for a Layer - the network parameter data required for a layer.
		 */
		struct ILayer
		{
			/** Virtual destructor */
			virtual ~ILayer() = default;

			/** Create the instance data required for this type of layer. */
			virtual TSharedPtr<ILayerInstance> MakeInstance() const { return nullptr; };

			/** Get the layer type. */
			virtual ELayerType GetLayerType() const = 0;

			/** Get the size of the input vector. */
			virtual uint32 GetInputSize() const = 0;

			/** Get the size of the output vector. */
			virtual uint32 GetOutputSize() const = 0;

			/** Compute the size required to serialize this layer by growing InOutOffset. */
			virtual void SerializationSize(uint64& InOutOffset) const = 0;

			/** Load this layer from the buffer at the given offset. */
			virtual void SerializationLoad(uint64& InOutOffset, TConstArrayView<uint8> Data) = 0;

			/** Save this layer from the buffer at the given offset. */
			virtual void SerializationSave(uint64& InOutOffset, TArrayView<uint8> Data) const = 0;

			/**
			 * Evaluate this layer.
			 *
			 * @param Instance				The instance data for this layer - what was returned by `MakeInstance`.
			 * @param OutputBuffer			The output buffer
			 * @param InputBuffer			The input buffer
			 * @param BatchSize				The number of items in the batch
			 * @param OutputBufferSize		The vector size of the items in the output
			 * @param InputBufferSize		The vector size of the items in the input
			 * @param OutputBufferStride	The stride of the output for each item in the batch
			 * @param InputBufferStride		The stride of the input for each item in the batch
			 */
			virtual void Evaluate(
				ILayerInstance* Instance,
				float* OutputBuffer,
				const float* InputBuffer,
				const uint32 BatchSize,
				const uint32 OutputBufferSize,
				const uint32 InputBufferSize,
				const uint32 OutputBufferStride,
				const uint32 InputBufferStride) = 0;
		};

		//--------------------------------------------------------------------------
		// Layers
		//--------------------------------------------------------------------------

		struct FSequenceLayer;

		struct FSequenceLayerInstance : public ILayerInstance
		{
			FSequenceLayerInstance(const FSequenceLayer& InSequenceLayer);

			virtual void SetMaxBatchSize(const uint32 MaxBatchSize) override final;
			virtual ELayerType GetLayerType() const override final { return ELayerType::Sequence; }

			const FSequenceLayer& SequenceLayer;
			uint32 ActivationStride = 0;
			TArray<TSharedPtr<ILayerInstance>, TInlineAllocator<32>> Instances;
			TArray<float> ActivationBufferFront;
			TArray<float> ActivationBufferBack;
		};

		struct FSequenceLayer : public ILayer
		{
			virtual TSharedPtr<ILayerInstance> MakeInstance() const { return MakeShared<FSequenceLayerInstance>(*this); };
			virtual ELayerType GetLayerType() const override final { return ELayerType::Sequence; }
			virtual uint32 GetInputSize() const override final { return Layers.Num() > 0 ? Layers[0]->GetInputSize() : 0; }
			virtual uint32 GetOutputSize() const override final { return Layers.Num() > 0 ? Layers.Last()->GetOutputSize() : 0; }

			virtual void SerializationSize(uint64& InOutOffset) const override final
			{
				Serialization::Size(InOutOffset, (uint32)Layers.Num());
				Serialization::Size(InOutOffset, Layers);
			}

			virtual void SerializationLoad(uint64& InOutOffset, TConstArrayView<uint8> Data) override final
			{
				uint32 LayerNum = 0;
				Serialization::Load(InOutOffset, LayerNum, Data);
				Layers.Init(nullptr, LayerNum);
				Serialization::Load(InOutOffset, Layers, Data);

				LayerInputSizes.SetNumUninitialized(LayerNum);
				LayerOutputSizes.SetNumUninitialized(LayerNum);
				for (uint32 LayerIdx = 0; LayerIdx < LayerNum; LayerIdx++)
				{
					LayerInputSizes[LayerIdx] = Layers[LayerIdx]->GetInputSize();
					LayerOutputSizes[LayerIdx] = Layers[LayerIdx]->GetOutputSize();
				}
			}

			virtual void SerializationSave(uint64& InOutOffset, TArrayView<uint8> Data) const override final
			{
				Serialization::Save(InOutOffset, (uint32)Layers.Num(), Data);
				Serialization::Save(InOutOffset, Layers, Data);
			}

			virtual void Evaluate(
				ILayerInstance* Instance,
				float* OutputBuffer,
				const float* InputBuffer,
				const uint32 BatchSize,
				const uint32 OutputBufferSize,
				const uint32 InputBufferSize,
				const uint32 OutputBufferStride,
				const uint32 InputBufferStride) override final
			{
				NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::FSequenceLayer::Evaluate);
				check(OutputBufferSize == GetOutputSize() && InputBufferSize == GetInputSize());
				check(OutputBufferStride >= GetOutputSize() && InputBufferStride >= GetInputSize());
				check(Instance && Instance->GetLayerType() == GetLayerType());

				FSequenceLayerInstance* SequenceInstance = StaticCast<FSequenceLayerInstance*>(Instance);

				OperatorNanCheck(InputBuffer, BatchSize, InputBufferSize, InputBufferStride);

				const uint32 LayerNum = Layers.Num();

				// If we just have one layer then evaluate layer directly without using intermediate storage

				if (LayerNum == 1)
				{
					Layers[0]->Evaluate(
						SequenceInstance->Instances[0].Get(),
						OutputBuffer,
						InputBuffer,
						BatchSize,
						OutputBufferSize,
						InputBufferSize,
						OutputBufferStride,
						InputBufferStride);

					return;
				}

				// Otherwise evaluate first layer from input into activation buffer

				Layers[0]->Evaluate(
					SequenceInstance->Instances[0].Get(),
					SequenceInstance->ActivationBufferFront.GetData(),
					InputBuffer,
					BatchSize,
					LayerOutputSizes[0],
					LayerInputSizes[0],
					SequenceInstance->ActivationStride,
					InputBufferStride);

				// Evaluate intermediate layers using front and back buffers

				for (uint32 LayerIdx = 1; LayerIdx < LayerNum - 1; LayerIdx++)
				{
					TConstArrayView<float> LayerInput = LayerIdx % 2 == 0 ? 
						SequenceInstance->ActivationBufferBack : 
						SequenceInstance->ActivationBufferFront;

					TArrayView<float> LayerOutput = LayerIdx % 2 == 0 ? 
						SequenceInstance->ActivationBufferFront : 
						SequenceInstance->ActivationBufferBack;

					Layers[LayerIdx]->Evaluate(
						SequenceInstance->Instances[LayerIdx].Get(),
						LayerOutput.GetData(),
						LayerInput.GetData(),
						BatchSize,
						LayerOutputSizes[LayerIdx],
						LayerInputSizes[LayerIdx],
						SequenceInstance->ActivationStride,
						SequenceInstance->ActivationStride);
				}

				// Evaluate final layer from activation buffer into output

				TConstArrayView<float> FinalLayerInput = LayerNum % 2 == 0 ? 
					SequenceInstance->ActivationBufferFront : 
					SequenceInstance->ActivationBufferBack;

				Layers.Last()->Evaluate(
					SequenceInstance->Instances.Last().Get(),
					OutputBuffer,
					FinalLayerInput.GetData(),
					BatchSize,
					OutputBufferSize,
					LayerInputSizes.Last(),
					OutputBufferStride,
					SequenceInstance->ActivationStride);

				OperatorNanCheck(OutputBuffer, BatchSize, OutputBufferSize, OutputBufferStride);
			}

			TArray<TSharedPtr<ILayer>, TInlineAllocator<32>> Layers;
			TArray<uint32, TInlineAllocator<32>> LayerInputSizes;
			TArray<uint32, TInlineAllocator<32>> LayerOutputSizes;
		};

		FSequenceLayerInstance::FSequenceLayerInstance(const FSequenceLayer& InSequenceLayer) 
			: SequenceLayer(InSequenceLayer)
		{
			const uint32 LayerNum = SequenceLayer.Layers.Num();
			Instances.Init(nullptr, LayerNum);

			for (uint32 LayerIdx = 0; LayerIdx < LayerNum; LayerIdx++)
			{
				Instances[LayerIdx] = SequenceLayer.Layers[LayerIdx]->MakeInstance();
			}

			// Compute the largest intermediate size used

			if (LayerNum == 0)
			{
				ActivationStride = 0;
			}
			else
			{
				ActivationStride = SequenceLayer.Layers[0]->GetOutputSize();
				for (uint32 LayerIdx = 1; LayerIdx < LayerNum - 1; LayerIdx++)
				{
					ActivationStride = FMath::Max(ActivationStride, SequenceLayer.Layers[LayerIdx]->GetOutputSize());
				}
			}
		}

		void FSequenceLayerInstance::SetMaxBatchSize(const uint32 MaxBatchSize)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::FSequenceLayerInstance::SetMaxBatchSize);

			// Propagate call to sub-layer instances

			for (const TSharedPtr<ILayerInstance>& Instance : Instances)
			{
				// Most layers don't allocate instance data and so we need to check for nullptr
				if (Instance)
				{
					Instance->SetMaxBatchSize(MaxBatchSize);
				}
			}

			// Allocate front and back buffers to maximum size. Don't shrink to avoid re-allocation 
			// when smaller batches are requested.

			ActivationBufferFront.SetNumUninitialized(MaxBatchSize * ActivationStride, EAllowShrinking::No);
			ActivationBufferBack.SetNumUninitialized(MaxBatchSize * ActivationStride, EAllowShrinking::No);
		}

		//--------------------------------------------------------------------------

		struct FNormalizeLayer : public ILayer
		{
			virtual ELayerType GetLayerType() const override final { return ELayerType::Normalize; }
			virtual uint32 GetInputSize() const override final { return InputOutputSize; }
			virtual uint32 GetOutputSize() const override final { return InputOutputSize; }

			virtual void SerializationSize(uint64& InOutOffset) const override final
			{
				Serialization::Size(InOutOffset, InputOutputSize);
				Serialization::Size(InOutOffset, Mean);
				Serialization::Size(InOutOffset, Std);
			}

			virtual void SerializationLoad(uint64& InOutOffset, TConstArrayView<uint8> Data) override final
			{
				Serialization::Load(InOutOffset, InputOutputSize, Data);
				Serialization::Load(InOutOffset, Mean, Data, InputOutputSize);
				Serialization::Load(InOutOffset, Std, Data, InputOutputSize);
			}

			virtual void SerializationSave(uint64& InOutOffset, TArrayView<uint8> Data) const override final
			{
				Serialization::Save(InOutOffset, InputOutputSize, Data);
				Serialization::Save(InOutOffset, Mean, Data);
				Serialization::Save(InOutOffset, Std, Data);
			}

			virtual void Evaluate(
				ILayerInstance* Instance,
				float* OutputBuffer,
				const float* InputBuffer,
				const uint32 BatchSize,
				const uint32 OutputBufferSize,
				const uint32 InputBufferSize,
				const uint32 OutputBufferStride,
				const uint32 InputBufferStride) override final
			{
				NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::FNormalizeLayer::Evaluate);
				check(OutputBufferSize == GetOutputSize() && InputBufferSize == GetInputSize());
				check(OutputBufferStride >= GetOutputSize() && InputBufferStride >= GetInputSize());
				check(Instance == nullptr);
				OperatorNanCheck(InputBuffer, BatchSize, InputBufferSize, InputBufferStride);

				OperatorNormalize(
					OutputBuffer,
					InputBuffer,
					Mean.GetData(),
					Std.GetData(),
					BatchSize,
					InputOutputSize,
					OutputBufferStride,
					InputBufferStride);

				OperatorNanCheck(OutputBuffer, BatchSize, OutputBufferSize, OutputBufferStride);
			}

			uint32 InputOutputSize = 0;
			TConstArrayView<float> Mean;
			TConstArrayView<float> Std;
		};

		//--------------------------------------------------------------------------

		struct FDenormalizeLayer : public ILayer
		{
			virtual ELayerType GetLayerType() const override final { return ELayerType::Denormalize; }
			virtual uint32 GetInputSize() const override final { return InputOutputSize; }
			virtual uint32 GetOutputSize() const override final { return InputOutputSize; }

			virtual void SerializationSize(uint64& InOutOffset) const override final
			{
				Serialization::Size(InOutOffset, InputOutputSize);
				Serialization::Size(InOutOffset, Mean);
				Serialization::Size(InOutOffset, Std);
			}

			virtual void SerializationLoad(uint64& InOutOffset, TConstArrayView<uint8> Data) override final
			{
				Serialization::Load(InOutOffset, InputOutputSize, Data);
				Serialization::Load(InOutOffset, Mean, Data, InputOutputSize);
				Serialization::Load(InOutOffset, Std, Data, InputOutputSize);
			}

			virtual void SerializationSave(uint64& InOutOffset, TArrayView<uint8> Data) const override final
			{
				Serialization::Save(InOutOffset, InputOutputSize, Data);
				Serialization::Save(InOutOffset, Mean, Data);
				Serialization::Save(InOutOffset, Std, Data);
			}

			virtual void Evaluate(
				ILayerInstance* Instance,
				float* OutputBuffer,
				const float* InputBuffer,
				const uint32 BatchSize,
				const uint32 OutputBufferSize,
				const uint32 InputBufferSize,
				const uint32 OutputBufferStride,
				const uint32 InputBufferStride) override final
			{
				NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::FDenormalizeLayer::Evaluate);
				check(OutputBufferSize == GetOutputSize() && InputBufferSize == GetInputSize());
				check(OutputBufferStride >= GetOutputSize() && InputBufferStride >= GetInputSize());
				check(Instance == nullptr);
				OperatorNanCheck(InputBuffer, BatchSize, InputBufferSize, InputBufferStride);

				OperatorDenormalize(
					OutputBuffer,
					InputBuffer,
					Mean.GetData(),
					Std.GetData(),
					BatchSize,
					InputOutputSize,
					OutputBufferStride,
					InputBufferStride);

				OperatorNanCheck(OutputBuffer, BatchSize, OutputBufferSize, OutputBufferStride);
			}

			uint32 InputOutputSize = 0;
			TConstArrayView<float> Mean;
			TConstArrayView<float> Std;
		};

		//--------------------------------------------------------------------------

		struct FLinearLayer : public ILayer
		{
			virtual ELayerType GetLayerType() const override final { return ELayerType::Linear; }
			virtual uint32 GetInputSize() const override final { return InputSize; }
			virtual uint32 GetOutputSize() const override final { return OutputSize; }

			virtual void SerializationSize(uint64& InOutOffset) const override final
			{
				Serialization::Size(InOutOffset, InputSize);
				Serialization::Size(InOutOffset, OutputSize);
				Serialization::Size(InOutOffset, Biases);
				Serialization::Size(InOutOffset, Weights);
			}

			virtual void SerializationLoad(uint64& InOutOffset, TConstArrayView<uint8> Data) override final
			{
				Serialization::Load(InOutOffset, InputSize, Data);
				Serialization::Load(InOutOffset, OutputSize, Data);
				Serialization::Load(InOutOffset, Biases, Data, OutputSize);
				Serialization::Load(InOutOffset, Weights, Data, InputSize * OutputSize);
			}

			virtual void SerializationSave(uint64& InOutOffset, TArrayView<uint8> Data) const override final
			{
				Serialization::Save(InOutOffset, InputSize, Data);
				Serialization::Save(InOutOffset, OutputSize, Data);
				Serialization::Save(InOutOffset, Biases, Data);
				Serialization::Save(InOutOffset, Weights, Data);
			}

			virtual void Evaluate(
				ILayerInstance* Instance,
				float* OutputBuffer,
				const float* InputBuffer,
				const uint32 BatchSize,
				const uint32 OutputBufferSize,
				const uint32 InputBufferSize,
				const uint32 OutputBufferStride,
				const uint32 InputBufferStride) override final
			{
				NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::FLinearLayer::Evaluate);
				check(OutputBufferSize == GetOutputSize() && InputBufferSize == GetInputSize());
				check(OutputBufferStride >= GetOutputSize() && InputBufferStride >= GetInputSize());
				check(Instance == nullptr);
				OperatorNanCheck(InputBuffer, BatchSize, InputBufferSize, InputBufferStride);

				OperatorLinear(
					OutputBuffer,
					InputBuffer,
					Weights.GetData(),
					Biases.GetData(),
					BatchSize,
					OutputSize,
					InputSize,
					OutputBufferStride,
					InputBufferStride);
				
				OperatorNanCheck(OutputBuffer, BatchSize, OutputBufferSize, OutputBufferStride);
			}

			uint32 InputSize = 0;
			uint32 OutputSize = 0;
			TConstArrayView<float> Biases;
			TConstArrayView<float> Weights;
		};

		//--------------------------------------------------------------------------

		struct FCompressedLinearLayer : public ILayer
		{
			virtual ELayerType GetLayerType() const override final { return ELayerType::CompressedLinear; }
			virtual uint32 GetInputSize() const override final { return InputSize; }
			virtual uint32 GetOutputSize() const override final { return OutputSize; }

			virtual void SerializationSize(uint64& InOutOffset) const override final
			{
				Serialization::Size(InOutOffset, InputSize);
				Serialization::Size(InOutOffset, OutputSize);
				Serialization::Size(InOutOffset, WeightOffsets);
				Serialization::Size(InOutOffset, WeightScales);
				Serialization::Size(InOutOffset, Biases);
				Serialization::Size(InOutOffset, Weights);
			}

			virtual void SerializationLoad(uint64& InOutOffset, TConstArrayView<uint8> Data) override final
			{
				Serialization::Load(InOutOffset, InputSize, Data);
				Serialization::Load(InOutOffset, OutputSize, Data);
				Serialization::Load(InOutOffset, WeightOffsets, Data, InputSize);
				Serialization::Load(InOutOffset, WeightScales, Data, InputSize);
				Serialization::Load(InOutOffset, Biases, Data, OutputSize);
				Serialization::Load(InOutOffset, Weights, Data, InputSize * OutputSize);
			}

			virtual void SerializationSave(uint64& InOutOffset, TArrayView<uint8> Data) const override final
			{
				Serialization::Save(InOutOffset, InputSize, Data);
				Serialization::Save(InOutOffset, OutputSize, Data);
				Serialization::Save(InOutOffset, WeightOffsets, Data);
				Serialization::Save(InOutOffset, WeightScales, Data);
				Serialization::Save(InOutOffset, Biases, Data);
				Serialization::Save(InOutOffset, Weights, Data);
			}

			virtual void Evaluate(
				ILayerInstance* Instance,
				float* OutputBuffer,
				const float* InputBuffer,
				const uint32 BatchSize,
				const uint32 OutputBufferSize,
				const uint32 InputBufferSize,
				const uint32 OutputBufferStride,
				const uint32 InputBufferStride) override final
			{
				NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::FCompressedLinearLayer::Evaluate);
				check(OutputBufferSize == GetOutputSize() && InputBufferSize == GetInputSize());
				check(OutputBufferStride >= GetOutputSize() && InputBufferStride >= GetInputSize());
				check(Instance == nullptr);
				OperatorNanCheck(InputBuffer, BatchSize, InputBufferSize, InputBufferStride);

				OperatorCompressedLinear(
					OutputBuffer,
					InputBuffer,
					Weights.GetData(),
					WeightOffsets.GetData(),
					WeightScales.GetData(),
					Biases.GetData(),
					BatchSize,
					OutputSize,
					InputSize,
					OutputBufferStride,
					InputBufferStride);

				OperatorNanCheck(OutputBuffer, BatchSize, OutputBufferSize, OutputBufferStride);
			}

			uint32 InputSize = 0;
			uint32 OutputSize = 0;
			TConstArrayView<float> WeightOffsets;
			TConstArrayView<float> WeightScales;
			TConstArrayView<float> Biases;
			TConstArrayView<uint16> Weights;
		};

		//--------------------------------------------------------------------------

		struct FMultiLinearLayer : public ILayer
		{
			virtual ELayerType GetLayerType() const override final { return ELayerType::MultiLinear; }
			virtual uint32 GetInputSize() const override final { return BlockNum * InputSize; }
			virtual uint32 GetOutputSize() const override final { return BlockNum * OutputSize; }

			virtual void SerializationSize(uint64& InOutOffset) const override final
			{
				Serialization::Size(InOutOffset, BlockNum);
				Serialization::Size(InOutOffset, InputSize);
				Serialization::Size(InOutOffset, OutputSize);
				Serialization::Size(InOutOffset, Biases);
				Serialization::Size(InOutOffset, Weights);
			}

			virtual void SerializationLoad(uint64& InOutOffset, TConstArrayView<uint8> Data) override final
			{
				Serialization::Load(InOutOffset, BlockNum, Data);
				Serialization::Load(InOutOffset, InputSize, Data);
				Serialization::Load(InOutOffset, OutputSize, Data);
				Serialization::Load(InOutOffset, Biases, Data, BlockNum * OutputSize);
				Serialization::Load(InOutOffset, Weights, Data, BlockNum * InputSize * OutputSize);
			}

			virtual void SerializationSave(uint64& InOutOffset, TArrayView<uint8> Data) const override final
			{
				Serialization::Save(InOutOffset, BlockNum, Data);
				Serialization::Save(InOutOffset, InputSize, Data);
				Serialization::Save(InOutOffset, OutputSize, Data);
				Serialization::Save(InOutOffset, Biases, Data);
				Serialization::Save(InOutOffset, Weights, Data);
			}

			virtual void Evaluate(
				ILayerInstance* Instance,
				float* OutputBuffer,
				const float* InputBuffer,
				const uint32 BatchSize,
				const uint32 OutputBufferSize,
				const uint32 InputBufferSize,
				const uint32 OutputBufferStride,
				const uint32 InputBufferStride) override final
			{
				NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::FMultiLinearLayer::Evaluate);
				check(OutputBufferSize == GetOutputSize() && InputBufferSize == GetInputSize());
				check(OutputBufferStride >= GetOutputSize() && InputBufferStride >= GetInputSize());
				check(Instance == nullptr);
				OperatorNanCheck(InputBuffer, BatchSize, InputBufferSize, InputBufferStride);

				OperatorMultiLinear(
					OutputBuffer,
					InputBuffer,
					Weights.GetData(),
					Biases.GetData(),
					BatchSize,
					BlockNum,
					OutputSize,
					InputSize,
					OutputBufferStride,
					InputBufferStride);

				OperatorNanCheck(OutputBuffer, BatchSize, OutputBufferSize, OutputBufferStride);
			}

			uint32 InputSize = 0;
			uint32 OutputSize = 0;
			uint32 BlockNum = 0;
			TConstArrayView<float> Biases;
			TConstArrayView<float> Weights;
		};

		//--------------------------------------------------------------------------

		struct FReLULayer : public ILayer
		{
			virtual ELayerType GetLayerType() const override final { return ELayerType::ReLU; }
			virtual uint32 GetInputSize() const override final { return InputOutputSize; }
			virtual uint32 GetOutputSize() const override final { return InputOutputSize; }

			virtual void SerializationSize(uint64& InOutOffset) const override final
			{
				Serialization::Size(InOutOffset, InputOutputSize);
			}

			virtual void SerializationLoad(uint64& InOutOffset, TConstArrayView<uint8> Data) override final
			{
				Serialization::Load(InOutOffset, InputOutputSize, Data);
			}

			virtual void SerializationSave(uint64& InOutOffset, TArrayView<uint8> Data) const override final
			{
				Serialization::Save(InOutOffset, InputOutputSize, Data);
			}

			virtual void Evaluate(
				ILayerInstance* Instance,
				float* OutputBuffer,
				const float* InputBuffer,
				const uint32 BatchSize,
				const uint32 OutputBufferSize,
				const uint32 InputBufferSize,
				const uint32 OutputBufferStride,
				const uint32 InputBufferStride) override final
			{
				NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::FReLULayer::Evaluate);
				check(OutputBufferSize == GetOutputSize() && InputBufferSize == GetInputSize());
				check(OutputBufferStride >= GetOutputSize() && InputBufferStride >= GetInputSize());
				check(Instance == nullptr);
				OperatorNanCheck(InputBuffer, BatchSize, InputBufferSize, InputBufferStride);

				OperatorReLU(
					OutputBuffer,
					InputBuffer,
					BatchSize,
					InputOutputSize,
					OutputBufferStride,
					InputBufferStride);

				OperatorNanCheck(OutputBuffer, BatchSize, OutputBufferSize, OutputBufferStride);
			}

			uint32 InputOutputSize = 0;
		};

		//--------------------------------------------------------------------------

		struct FELULayer : public ILayer
		{
			virtual ELayerType GetLayerType() const override final { return ELayerType::ELU; }
			virtual uint32 GetInputSize() const override final { return InputOutputSize; }
			virtual uint32 GetOutputSize() const override final { return InputOutputSize; }

			virtual void SerializationSize(uint64& InOutOffset) const override final
			{
				Serialization::Size(InOutOffset, InputOutputSize);
			}

			virtual void SerializationLoad(uint64& InOutOffset, TConstArrayView<uint8> Data) override final
			{
				Serialization::Load(InOutOffset, InputOutputSize, Data);
			}

			virtual void SerializationSave(uint64& InOutOffset, TArrayView<uint8> Data) const override final
			{
				Serialization::Save(InOutOffset, InputOutputSize, Data);
			}

			virtual void Evaluate(
				ILayerInstance* Instance,
				float* OutputBuffer,
				const float* InputBuffer,
				const uint32 BatchSize,
				const uint32 OutputBufferSize,
				const uint32 InputBufferSize,
				const uint32 OutputBufferStride,
				const uint32 InputBufferStride) override final
			{
				NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::FELULayer::Evaluate);
				check(OutputBufferSize == GetOutputSize() && InputBufferSize == GetInputSize());
				check(OutputBufferStride >= GetOutputSize() && InputBufferStride >= GetInputSize());
				check(Instance == nullptr);
				OperatorNanCheck(InputBuffer, BatchSize, InputBufferSize, InputBufferStride);

				OperatorELU(
					OutputBuffer,
					InputBuffer,
					BatchSize,
					InputOutputSize,
					OutputBufferStride,
					InputBufferStride);

				OperatorNanCheck(OutputBuffer, BatchSize, OutputBufferSize, OutputBufferStride);
			}

			uint32 InputOutputSize = 0;
		};

		//--------------------------------------------------------------------------

		struct FGELULayer : public ILayer
		{
			virtual ELayerType GetLayerType() const override final { return ELayerType::GELU; }
			virtual uint32 GetInputSize() const override final { return InputOutputSize; }
			virtual uint32 GetOutputSize() const override final { return InputOutputSize; }

			virtual void SerializationSize(uint64& InOutOffset) const override final
			{
				Serialization::Size(InOutOffset, InputOutputSize);
			}

			virtual void SerializationLoad(uint64& InOutOffset, TConstArrayView<uint8> Data) override final
			{
				Serialization::Load(InOutOffset, InputOutputSize, Data);
			}

			virtual void SerializationSave(uint64& InOutOffset, TArrayView<uint8> Data) const override final
			{
				Serialization::Save(InOutOffset, InputOutputSize, Data);
			}

			virtual void Evaluate(
				ILayerInstance* Instance,
				float* OutputBuffer,
				const float* InputBuffer,
				const uint32 BatchSize,
				const uint32 OutputBufferSize,
				const uint32 InputBufferSize,
				const uint32 OutputBufferStride,
				const uint32 InputBufferStride) override final
			{
				NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::FGELULayer::Evaluate);
				check(OutputBufferSize == GetOutputSize() && InputBufferSize == GetInputSize());
				check(OutputBufferStride >= GetOutputSize() && InputBufferStride >= GetInputSize());
				check(Instance == nullptr);
				OperatorNanCheck(InputBuffer, BatchSize, InputBufferSize, InputBufferStride);

				OperatorGELU(
					OutputBuffer,
					InputBuffer,
					BatchSize,
					InputOutputSize,
					OutputBufferStride,
					InputBufferStride);

				OperatorNanCheck(OutputBuffer, BatchSize, OutputBufferSize, OutputBufferStride);
			}

			uint32 InputOutputSize = 0;
		};

		//--------------------------------------------------------------------------

		struct FTanHLayer : public ILayer
		{
			virtual ELayerType GetLayerType() const override final { return ELayerType::TanH; }
			virtual uint32 GetInputSize() const override final { return InputOutputSize; }
			virtual uint32 GetOutputSize() const override final { return InputOutputSize; }

			virtual void SerializationSize(uint64& InOutOffset) const override final
			{
				Serialization::Size(InOutOffset, InputOutputSize);
			}

			virtual void SerializationLoad(uint64& InOutOffset, TConstArrayView<uint8> Data) override final
			{
				Serialization::Load(InOutOffset, InputOutputSize, Data);
			}

			virtual void SerializationSave(uint64& InOutOffset, TArrayView<uint8> Data) const override final
			{
				Serialization::Save(InOutOffset, InputOutputSize, Data);
			}

			virtual void Evaluate(
				ILayerInstance* Instance,
				float* OutputBuffer,
				const float* InputBuffer,
				const uint32 BatchSize,
				const uint32 OutputBufferSize,
				const uint32 InputBufferSize,
				const uint32 OutputBufferStride,
				const uint32 InputBufferStride) override final
			{
				NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::FTanHLayer::Evaluate);
				check(OutputBufferSize == GetOutputSize() && InputBufferSize == GetInputSize());
				check(OutputBufferStride >= GetOutputSize() && InputBufferStride >= GetInputSize());
				check(Instance == nullptr);
				OperatorNanCheck(InputBuffer, BatchSize, InputBufferSize, InputBufferStride);

				OperatorTanH(
					OutputBuffer,
					InputBuffer,
					BatchSize,
					InputOutputSize,
					OutputBufferStride,
					InputBufferStride);

				OperatorNanCheck(OutputBuffer, BatchSize, OutputBufferSize, OutputBufferStride);
			}

			uint32 InputOutputSize = 0;
		};

		//--------------------------------------------------------------------------

		struct FPReLULayer : public ILayer
		{
			virtual ELayerType GetLayerType() const override final { return ELayerType::PReLU; }
			virtual uint32 GetInputSize() const override final { return InputOutputSize; }
			virtual uint32 GetOutputSize() const override final { return InputOutputSize; }

			virtual void SerializationSize(uint64& InOutOffset) const override final
			{
				Serialization::Size(InOutOffset, InputOutputSize);
				Serialization::Size(InOutOffset, Alpha);
			}

			virtual void SerializationLoad(uint64& InOutOffset, TConstArrayView<uint8> Data) override final
			{
				Serialization::Load(InOutOffset, InputOutputSize, Data);
				Serialization::Load(InOutOffset, Alpha, Data, InputOutputSize);
			}

			virtual void SerializationSave(uint64& InOutOffset, TArrayView<uint8> Data) const override final
			{
				Serialization::Save(InOutOffset, InputOutputSize, Data);
				Serialization::Save(InOutOffset, Alpha, Data);
			}

			virtual void Evaluate(
				ILayerInstance* Instance,
				float* OutputBuffer,
				const float* InputBuffer,
				const uint32 BatchSize,
				const uint32 OutputBufferSize,
				const uint32 InputBufferSize,
				const uint32 OutputBufferStride,
				const uint32 InputBufferStride) override final
			{
				NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::FPReLuLayer::Evaluate);
				check(OutputBufferSize == GetOutputSize() && InputBufferSize == GetInputSize());
				check(OutputBufferStride >= GetOutputSize() && InputBufferStride >= GetInputSize());
				check(Instance == nullptr);
				OperatorNanCheck(InputBuffer, BatchSize, InputBufferSize, InputBufferStride);

				OperatorPReLU(
					OutputBuffer,
					InputBuffer,
					Alpha.GetData(),
					BatchSize,
					InputOutputSize,
					OutputBufferStride,
					InputBufferStride);

				OperatorNanCheck(OutputBuffer, BatchSize, OutputBufferSize, OutputBufferStride);
			}

			uint32 InputOutputSize = 0;
			TConstArrayView<float> Alpha;
		};

		//--------------------------------------------------------------------------

		struct FMemoryCellLayer;

		struct FMemoryCellInstance : public ILayerInstance
		{
			FMemoryCellInstance(const FMemoryCellLayer& InMemoryCellLayer);

			virtual void SetMaxBatchSize(const uint32 MaxBatchSize) override final;
			virtual ELayerType GetLayerType() const override final { return ELayerType::MemoryCell; }

			const FMemoryCellLayer& MemoryCellLayer;
			TSharedPtr<ILayerInstance> RememberInstance;
			TSharedPtr<ILayerInstance> PassthroughInstance;
			TSharedPtr<ILayerInstance> MemoryUpdateInstance;
			TSharedPtr<ILayerInstance> OutputInputUpdateInstance;
			TSharedPtr<ILayerInstance> OutputMemoryUpdateInstance;
			TArray<float> RememberGateBuffer;
			TArray<float> MemoryUpdateBuffer;
			TArray<float> PassthroughGateBuffer;
			TArray<float> OutputMemoryUpdateBuffer;
			TArray<float> OutputInputUpdateBuffer;
		};

		struct FMemoryCellLayer : public ILayer
		{
			virtual TSharedPtr<ILayerInstance> MakeInstance() const { return MakeShared<FMemoryCellInstance>(*this); };
			virtual ELayerType GetLayerType() const override final { return ELayerType::MemoryCell; }
			virtual uint32 GetInputSize() const override final { return InputSize + MemorySize; }
			virtual uint32 GetOutputSize() const override final { return OutputSize + MemorySize; }

			virtual void SerializationSize(uint64& InOutOffset) const override final
			{
				Serialization::Size(InOutOffset, InputSize);
				Serialization::Size(InOutOffset, OutputSize);
				Serialization::Size(InOutOffset, MemorySize);
				Serialization::Size(InOutOffset, RememberLayer);
				Serialization::Size(InOutOffset, PassthroughLayer);
				Serialization::Size(InOutOffset, MemoryUpdateLayer);
				Serialization::Size(InOutOffset, OutputInputUpdateLayer);
				Serialization::Size(InOutOffset, OutputMemoryUpdateLayer);
			}

			virtual void SerializationLoad(uint64& InOutOffset, TConstArrayView<uint8> Data) override final
			{
				Serialization::Load(InOutOffset, InputSize, Data);
				Serialization::Load(InOutOffset, OutputSize, Data);
				Serialization::Load(InOutOffset, MemorySize, Data);
				Serialization::Load(InOutOffset, RememberLayer, Data);
				Serialization::Load(InOutOffset, PassthroughLayer, Data);
				Serialization::Load(InOutOffset, MemoryUpdateLayer, Data);
				Serialization::Load(InOutOffset, OutputInputUpdateLayer, Data);
				Serialization::Load(InOutOffset, OutputMemoryUpdateLayer, Data);
			}

			virtual void SerializationSave(uint64& InOutOffset, TArrayView<uint8> Data) const override final
			{
				Serialization::Save(InOutOffset, InputSize, Data);
				Serialization::Save(InOutOffset, OutputSize, Data);
				Serialization::Save(InOutOffset, MemorySize, Data);
				Serialization::Save(InOutOffset, RememberLayer, Data);
				Serialization::Save(InOutOffset, PassthroughLayer, Data);
				Serialization::Save(InOutOffset, MemoryUpdateLayer, Data);
				Serialization::Save(InOutOffset, OutputInputUpdateLayer, Data);
				Serialization::Save(InOutOffset, OutputMemoryUpdateLayer, Data);
			}

			virtual void Evaluate(
				ILayerInstance* Instance,
				float* OutputBuffer,
				const float* InputBuffer,
				const uint32 BatchSize,
				const uint32 OutputBufferSize,
				const uint32 InputBufferSize,
				const uint32 OutputBufferStride,
				const uint32 InputBufferStride) override final
			{
				NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::FMemoryCellLayer::Evaluate);
				check(OutputBufferSize == GetOutputSize() && InputBufferSize == GetInputSize());
				check(OutputBufferStride >= GetOutputSize() && InputBufferStride >= GetInputSize());
				check(Instance && Instance->GetLayerType() == GetLayerType());

				FMemoryCellInstance* MemoryCellInstance = StaticCast<FMemoryCellInstance*>(Instance);
				
				OperatorNanCheck(InputBuffer, BatchSize, InputBufferSize, InputBufferStride);

				// Remember Gate

				RememberLayer->Evaluate(
					MemoryCellInstance->RememberInstance.Get(),
					MemoryCellInstance->RememberGateBuffer.GetData(),
					InputBuffer,
					BatchSize,
					MemorySize,
					InputSize + MemorySize,
					MemorySize,
					InputBufferStride);

				// Passthrough Gate

				PassthroughLayer->Evaluate(
					MemoryCellInstance->PassthroughInstance.Get(),
					MemoryCellInstance->PassthroughGateBuffer.GetData(),
					InputBuffer,
					BatchSize,
					OutputSize,
					InputSize + MemorySize,
					OutputSize,
					InputBufferStride);

				// Memory Update

				MemoryUpdateLayer->Evaluate(
					MemoryCellInstance->MemoryUpdateInstance.Get(),
					MemoryCellInstance->MemoryUpdateBuffer.GetData(),
					InputBuffer,
					BatchSize,
					MemorySize,
					InputSize + MemorySize,
					MemorySize,
					InputBufferStride);

				// Update Memory State

				OperatorMemoryCellUpdateMemory(
					OutputBuffer + OutputSize,
					MemoryCellInstance->RememberGateBuffer.GetData(),
					InputBuffer + InputSize,
					MemoryCellInstance->MemoryUpdateBuffer.GetData(),
					BatchSize,
					MemorySize,
					OutputBufferStride,
					MemorySize,
					InputBufferStride,
					MemorySize);

				// Output Input Update

				OutputInputUpdateLayer->Evaluate(
					MemoryCellInstance->OutputInputUpdateInstance.Get(),
					MemoryCellInstance->OutputInputUpdateBuffer.GetData(),
					InputBuffer,
					BatchSize,
					OutputSize,
					InputSize + MemorySize,
					OutputSize,
					InputBufferStride);

				// Output Memory Update

				OutputMemoryUpdateLayer->Evaluate(
					MemoryCellInstance->OutputMemoryUpdateInstance.Get(),
					MemoryCellInstance->OutputMemoryUpdateBuffer.GetData(),
					OutputBuffer + OutputSize,
					BatchSize,
					OutputSize,
					MemorySize,
					OutputSize,
					OutputBufferStride);

				// Update Final Output

				OperatorMemoryCellUpdateOutput(
					OutputBuffer,
					MemoryCellInstance->PassthroughGateBuffer.GetData(),
					MemoryCellInstance->OutputMemoryUpdateBuffer.GetData(),
					MemoryCellInstance->OutputInputUpdateBuffer.GetData(),
					BatchSize,
					OutputSize,
					OutputBufferStride,
					OutputSize,
					OutputSize,
					OutputSize);

				OperatorNanCheck(OutputBuffer, BatchSize, OutputBufferSize, OutputBufferStride);
			}

			uint32 InputSize = 0;
			uint32 OutputSize = 0;
			uint32 MemorySize = 0;
			TSharedPtr<ILayer> RememberLayer;
			TSharedPtr<ILayer> PassthroughLayer;
			TSharedPtr<ILayer> MemoryUpdateLayer;
			TSharedPtr<ILayer> OutputInputUpdateLayer;
			TSharedPtr<ILayer> OutputMemoryUpdateLayer;
		};

		FMemoryCellInstance::FMemoryCellInstance(const FMemoryCellLayer& InMemoryCellLayer) 
			: MemoryCellLayer(InMemoryCellLayer)
		{
			RememberInstance = MemoryCellLayer.RememberLayer->MakeInstance();
			PassthroughInstance = MemoryCellLayer.PassthroughLayer->MakeInstance();
			MemoryUpdateInstance = MemoryCellLayer.MemoryUpdateLayer->MakeInstance();
			OutputInputUpdateInstance = MemoryCellLayer.OutputInputUpdateLayer->MakeInstance();
			OutputMemoryUpdateInstance = MemoryCellLayer.OutputMemoryUpdateLayer->MakeInstance();
		}

		void FMemoryCellInstance::SetMaxBatchSize(const uint32 MaxBatchSize)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::FMemoryCellInstance::SetMaxBatchSize);

			if (RememberInstance) { RememberInstance->SetMaxBatchSize(MaxBatchSize); }
			if (PassthroughInstance) { PassthroughInstance->SetMaxBatchSize(MaxBatchSize); }
			if (MemoryUpdateInstance) { MemoryUpdateInstance->SetMaxBatchSize(MaxBatchSize); }
			if (OutputInputUpdateInstance) { OutputInputUpdateInstance->SetMaxBatchSize(MaxBatchSize); }
			if (OutputMemoryUpdateInstance) { OutputMemoryUpdateInstance->SetMaxBatchSize(MaxBatchSize); }

			RememberGateBuffer.SetNumUninitialized(MaxBatchSize * MemoryCellLayer.MemorySize, EAllowShrinking::No);
			PassthroughGateBuffer.SetNumUninitialized(MaxBatchSize * MemoryCellLayer.OutputSize, EAllowShrinking::No);
			MemoryUpdateBuffer.SetNumUninitialized(MaxBatchSize * MemoryCellLayer.MemorySize, EAllowShrinking::No);
			OutputInputUpdateBuffer.SetNumUninitialized(MaxBatchSize * MemoryCellLayer.OutputSize, EAllowShrinking::No);
			OutputMemoryUpdateBuffer.SetNumUninitialized(MaxBatchSize * MemoryCellLayer.OutputSize, EAllowShrinking::No);
		}

		//--------------------------------------------------------------------------

		struct FCopyLayer : public ILayer
		{
			virtual ELayerType GetLayerType() const override final { return ELayerType::Copy; }
			virtual uint32 GetInputSize() const override final { return InputOutputSize; }
			virtual uint32 GetOutputSize() const override final { return InputOutputSize; }

			virtual void SerializationSize(uint64& InOutOffset) const override final
			{
				Serialization::Size(InOutOffset, InputOutputSize);
			}

			virtual void SerializationLoad(uint64& InOutOffset, TConstArrayView<uint8> Data) override final
			{
				Serialization::Load(InOutOffset, InputOutputSize, Data);
			}

			virtual void SerializationSave(uint64& InOutOffset, TArrayView<uint8> Data) const override final
			{
				Serialization::Save(InOutOffset, InputOutputSize, Data);
			}

			virtual void Evaluate(
				ILayerInstance* Instance,
				float* OutputBuffer,
				const float* InputBuffer,
				const uint32 BatchSize,
				const uint32 OutputBufferSize,
				const uint32 InputBufferSize,
				const uint32 OutputBufferStride,
				const uint32 InputBufferStride) override final
			{
				NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::FCopyLayer::Evaluate);
				check(OutputBufferSize == GetOutputSize() && InputBufferSize == GetInputSize());
				check(OutputBufferStride >= GetOutputSize() && InputBufferStride >= GetInputSize());
				check(Instance == nullptr);
				OperatorNanCheck(InputBuffer, BatchSize, InputBufferSize, InputBufferStride);

				OperatorCopy(
					OutputBuffer,
					InputBuffer,
					BatchSize,
					InputOutputSize,
					OutputBufferStride,
					InputBufferStride);

				OperatorNanCheck(OutputBuffer, BatchSize, OutputBufferSize, OutputBufferStride);
			}

			uint32 InputOutputSize = 0;
		};

		//--------------------------------------------------------------------------

		struct FConcatLayer;

		struct FConcatLayerInstance : public ILayerInstance
		{
			FConcatLayerInstance(const FConcatLayer& InConcatLayer);

			virtual void SetMaxBatchSize(const uint32 MaxBatchSize) override final;
			virtual ELayerType GetLayerType() const override final { return ELayerType::Concat; }

			const FConcatLayer& ConcatLayer;
			TArray<TSharedPtr<ILayerInstance>, TInlineAllocator<32>> Instances;
		};

		struct FConcatLayer : public ILayer
		{
			virtual TSharedPtr<ILayerInstance> MakeInstance() const { return MakeShared<FConcatLayerInstance>(*this); };
			virtual ELayerType GetLayerType() const override final { return ELayerType::Concat; }
			virtual uint32 GetInputSize() const override final { return TotalInputSize; }
			virtual uint32 GetOutputSize() const override final { return TotalOutputSize; }

			virtual void SerializationSize(uint64& InOutOffset) const override final
			{
				Serialization::Size(InOutOffset, (uint32)Layers.Num());
				Serialization::Size(InOutOffset, InputSizes);
				Serialization::Size(InOutOffset, OutputSizes);
				Serialization::Size(InOutOffset, Layers);
			}

			virtual void SerializationLoad(uint64& InOutOffset, TConstArrayView<uint8> Data) override final
			{
				uint32 LayerNum = 0;
				Serialization::Load(InOutOffset, LayerNum, Data);
				Serialization::Load(InOutOffset, InputSizes, Data, LayerNum);
				Serialization::Load(InOutOffset, OutputSizes, Data, LayerNum);
				Layers.Init(nullptr, LayerNum);
				Serialization::Load(InOutOffset, Layers, Data);
				PostLoad();
			}

			void PostLoad()
			{
				const uint32 LayerNum = Layers.Num();
				InputOffsets.SetNumUninitialized(LayerNum);
				OutputOffsets.SetNumUninitialized(LayerNum);

				TotalInputSize = 0;
				TotalOutputSize = 0;
				for (uint32 LayerIdx = 0; LayerIdx < LayerNum; LayerIdx++)
				{
					InputOffsets[LayerIdx] = TotalInputSize;
					OutputOffsets[LayerIdx] = TotalOutputSize;
					TotalInputSize += InputSizes[LayerIdx];
					TotalOutputSize += OutputSizes[LayerIdx];
				}
			}

			virtual void SerializationSave(uint64& InOutOffset, TArrayView<uint8> Data) const override final
			{
				Serialization::Save(InOutOffset, (uint32)Layers.Num(), Data);
				Serialization::Save(InOutOffset, InputSizes, Data);
				Serialization::Save(InOutOffset, OutputSizes, Data);
				Serialization::Save(InOutOffset, Layers, Data);
			}

			virtual void Evaluate(
				ILayerInstance* Instance,
				float* OutputBuffer,
				const float* InputBuffer,
				const uint32 BatchSize,
				const uint32 OutputBufferSize,
				const uint32 InputBufferSize,
				const uint32 OutputBufferStride,
				const uint32 InputBufferStride) override final
			{
				NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::FConcatLayer::Evaluate);
				check(OutputBufferSize == GetOutputSize() && InputBufferSize == GetInputSize());
				check(OutputBufferStride >= GetOutputSize() && InputBufferStride >= GetInputSize());
				check(Instance && Instance->GetLayerType() == GetLayerType());

				FConcatLayerInstance* ConcatInstance = StaticCast<FConcatLayerInstance*>(Instance);

				OperatorNanCheck(InputBuffer, BatchSize, InputBufferSize, InputBufferStride);

				const int32 LayerNum = Layers.Num();

				for (int32 LayerIdx = 0; LayerIdx < LayerNum; LayerIdx++)
				{
					check(InputOffsets[LayerIdx] + InputSizes[LayerIdx] <= InputBufferSize);
					check(OutputOffsets[LayerIdx] + OutputSizes[LayerIdx] <= OutputBufferSize);

					Layers[LayerIdx]->Evaluate(
						ConcatInstance->Instances[LayerIdx].Get(),
						OutputBuffer + OutputOffsets[LayerIdx],
						InputBuffer + InputOffsets[LayerIdx],
						BatchSize,
						OutputSizes[LayerIdx],
						InputSizes[LayerIdx],
						OutputBufferStride,
						InputBufferStride);
				}

				OperatorNanCheck(OutputBuffer, BatchSize, OutputBufferSize, OutputBufferStride);
			}

			TConstArrayView<uint32> InputSizes;
			TConstArrayView<uint32> OutputSizes;
			TArray<TSharedPtr<ILayer>, TInlineAllocator<32>> Layers;

			uint32 TotalInputSize = 0;
			uint32 TotalOutputSize = 0;
			TArray<uint32, TInlineAllocator<32>> InputOffsets;
			TArray<uint32, TInlineAllocator<32>> OutputOffsets;
		};

		FConcatLayerInstance::FConcatLayerInstance(const FConcatLayer& InConcatLayer)
			: ConcatLayer(InConcatLayer)
		{
			const uint32 LayerNum = ConcatLayer.Layers.Num();
			Instances.Init(nullptr, LayerNum);

			for (uint32 LayerIdx = 0; LayerIdx < LayerNum; LayerIdx++)
			{
				Instances[LayerIdx] = ConcatLayer.Layers[LayerIdx]->MakeInstance();
			}
		}

		void FConcatLayerInstance::SetMaxBatchSize(const uint32 MaxBatchSize)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::FConcatLayerInstance::SetMaxBatchSize);

			// Propagate call to sub-layer instances

			for (const TSharedPtr<ILayerInstance>& Instance : Instances)
			{
				if (Instance)
				{
					Instance->SetMaxBatchSize(MaxBatchSize);
				}
			}
		}

		//--------------------------------------------------------------------------

		struct FArrayLayer;

		struct FArrayLayerInstance : public ILayerInstance
		{
			FArrayLayerInstance(const FArrayLayer& InArrayLayer);

			virtual void SetMaxBatchSize(const uint32 MaxBatchSize) override final;
			virtual ELayerType GetLayerType() const override final { return ELayerType::Array; }

			const FArrayLayer& ArrayLayer;
			TSharedPtr<ILayerInstance> Instance;
			TArray<float> ElementInputBuffer;
			TArray<float> ElementOutputBuffer;
		};

		struct FArrayLayer : public ILayer
		{
			virtual TSharedPtr<ILayerInstance> MakeInstance() const { return MakeShared<FArrayLayerInstance>(*this); };
			virtual ELayerType GetLayerType() const override final { return ELayerType::Array; }
			virtual uint32 GetInputSize() const override final { return ElementNum * ElementInputSize; }
			virtual uint32 GetOutputSize() const override final { return ElementNum * ElementOutputSize; }

			virtual void SerializationSize(uint64& InOutOffset) const override final
			{
				Serialization::Size(InOutOffset, ElementNum);
				Serialization::Size(InOutOffset, ElementInputSize);
				Serialization::Size(InOutOffset, ElementOutputSize);
				Serialization::Size(InOutOffset, SubLayer);
			}

			virtual void SerializationLoad(uint64& InOutOffset, TConstArrayView<uint8> Data) override final
			{
				Serialization::Load(InOutOffset, ElementNum, Data);
				Serialization::Load(InOutOffset, ElementInputSize, Data);
				Serialization::Load(InOutOffset, ElementOutputSize, Data);
				Serialization::Load(InOutOffset, SubLayer, Data);
			}

			virtual void SerializationSave(uint64& InOutOffset, TArrayView<uint8> Data) const override final
			{
				Serialization::Save(InOutOffset, ElementNum, Data);
				Serialization::Save(InOutOffset, ElementInputSize, Data);
				Serialization::Save(InOutOffset, ElementOutputSize, Data);
				Serialization::Save(InOutOffset, SubLayer, Data);
			}

			virtual void Evaluate(
				ILayerInstance* Instance,
				float* OutputBuffer,
				const float* InputBuffer,
				const uint32 BatchSize,
				const uint32 OutputBufferSize,
				const uint32 InputBufferSize,
				const uint32 OutputBufferStride,
				const uint32 InputBufferStride) override final
			{
				NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::FArrayLayer::Evaluate);
				check(OutputBufferSize == GetOutputSize() && InputBufferSize == GetInputSize());
				check(OutputBufferStride >= GetOutputSize() && InputBufferStride >= GetInputSize());
				check(Instance && Instance->GetLayerType() == GetLayerType());

				FArrayLayerInstance* ArrayInstance = StaticCast<FArrayLayerInstance*>(Instance);

				OperatorNanCheck(InputBuffer, BatchSize, InputBufferSize, InputBufferStride);

				// If inputs and outputs are already tightly packed then evaluate directly as large batch

				if (InputBufferStride == ElementNum * ElementInputSize &&
					OutputBufferStride == ElementNum * ElementOutputSize)
				{
					SubLayer->Evaluate(
						ArrayInstance->Instance.Get(),
						OutputBuffer,
						InputBuffer,
						BatchSize * ElementNum,
						ElementOutputSize,
						ElementInputSize,
						ElementOutputSize,
						ElementInputSize);

					return;
				}

				// Otherwise gather all inputs into one large buffer packed together tightly

				OperatorCopy(
					ArrayInstance->ElementInputBuffer.GetData(),
					InputBuffer,
					BatchSize,
					ElementNum * ElementInputSize,
					ElementNum * ElementInputSize,
					InputBufferStride);

				// Evaluate sub-layer on large batch of all elements

				SubLayer->Evaluate(
					ArrayInstance->Instance.Get(),
					ArrayInstance->ElementOutputBuffer.GetData(),
					ArrayInstance->ElementInputBuffer.GetData(),
					BatchSize * ElementNum,
					ElementOutputSize,
					ElementInputSize,
					ElementOutputSize,
					ElementInputSize);

				// And scatter outputs out of tightly packed buffer

				OperatorCopy(
					OutputBuffer,
					ArrayInstance->ElementOutputBuffer.GetData(),
					BatchSize,
					ElementNum * ElementOutputSize,
					OutputBufferStride,
					ElementNum * ElementOutputSize);

				OperatorNanCheck(OutputBuffer, BatchSize, OutputBufferSize, OutputBufferStride);
			}

			uint32 ElementNum = 0;
			uint32 ElementInputSize = 0;
			uint32 ElementOutputSize = 0;
			TSharedPtr<ILayer> SubLayer;
		};

		FArrayLayerInstance::FArrayLayerInstance(const FArrayLayer& InArrayLayer) : ArrayLayer(InArrayLayer)
		{
			Instance = ArrayLayer.SubLayer->MakeInstance();
		}

		void FArrayLayerInstance::SetMaxBatchSize(const uint32 MaxBatchSize)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::FArrayLayerInstance::SetMaxBatchSize);

			if (Instance)
			{
				// We are going to evaluate the sublayer on one large batch so we use MaxBatchSize * ArrayLayer.ElementNum.
				Instance->SetMaxBatchSize(MaxBatchSize * ArrayLayer.ElementNum);
			}

			ElementInputBuffer.SetNumUninitialized(MaxBatchSize * ArrayLayer.ElementNum * ArrayLayer.ElementInputSize, EAllowShrinking::No);
			ElementOutputBuffer.SetNumUninitialized(MaxBatchSize * ArrayLayer.ElementNum * ArrayLayer.ElementOutputSize, EAllowShrinking::No);
		}

		//--------------------------------------------------------------------------

		struct FAggregateSetLayer;

		struct FAggregateSetLayerInstance : public ILayerInstance
		{
			FAggregateSetLayerInstance(const FAggregateSetLayer& InAggregateSetLayer);

			virtual void SetMaxBatchSize(const uint32 MaxBatchSize) override final;
			virtual ELayerType GetLayerType() const override final { return ELayerType::AggregateSet; }

			const FAggregateSetLayer& AggregateSetLayer;
			TSharedPtr<ILayerInstance> SubLayerInstance;
			TSharedPtr<ILayerInstance> QueryInstance;
			TSharedPtr<ILayerInstance> KeyInstance;
			TSharedPtr<ILayerInstance> ValueInstance;

			uint32 TotalElementNum = 0;
			TArray<uint32, TInlineAllocator<32>> ElementNums;
			TArray<uint32, TInlineAllocator<32>> ElementOffsets;

			TArray<float> InputElementBuffer;
			TArray<float> OutputElementBuffer;
			TArray<float> QueryBuffer;
			TArray<float> KeyBuffer;
			TArray<float> ValueBuffer;
			TArray<float> AttentionMaxsBuffer;
			TArray<float> AttentionDenomsBuffer;
			TArray<float> AttentionBuffer;
		};

		struct FAggregateSetLayer : public ILayer
		{
			virtual TSharedPtr<ILayerInstance> MakeInstance() const { return MakeShared<FAggregateSetLayerInstance>(*this); };
			virtual ELayerType GetLayerType() const override final { return ELayerType::AggregateSet; }
			virtual uint32 GetInputSize() const override final { return MaxElementNum * ElementInputSize + MaxElementNum; }
			virtual uint32 GetOutputSize() const override final { return AttentionHeadNum * OutputEncodingSize + 1; }

			virtual void SerializationSize(uint64& InOutOffset) const override final
			{
				Serialization::Size(InOutOffset, MaxElementNum);
				Serialization::Size(InOutOffset, ElementInputSize);
				Serialization::Size(InOutOffset, ElementOutputSize);
				Serialization::Size(InOutOffset, OutputEncodingSize);
				Serialization::Size(InOutOffset, AttentionEncodingSize);
				Serialization::Size(InOutOffset, AttentionHeadNum);
				Serialization::Size(InOutOffset, SubLayer);
				Serialization::Size(InOutOffset, QueryLayer);
				Serialization::Size(InOutOffset, KeyLayer);
				Serialization::Size(InOutOffset, ValueLayer);
			}

			virtual void SerializationLoad(uint64& InOutOffset, TConstArrayView<uint8> Data) override final
			{
				Serialization::Load(InOutOffset, MaxElementNum, Data);
				Serialization::Load(InOutOffset, ElementInputSize, Data);
				Serialization::Load(InOutOffset, ElementOutputSize, Data);
				Serialization::Load(InOutOffset, OutputEncodingSize, Data);
				Serialization::Load(InOutOffset, AttentionEncodingSize, Data);
				Serialization::Load(InOutOffset, AttentionHeadNum, Data);
				Serialization::Load(InOutOffset, SubLayer, Data);
				Serialization::Load(InOutOffset, QueryLayer, Data);
				Serialization::Load(InOutOffset, KeyLayer, Data);
				Serialization::Load(InOutOffset, ValueLayer, Data);
			}

			virtual void SerializationSave(uint64& InOutOffset, TArrayView<uint8> Data) const override final
			{
				Serialization::Save(InOutOffset, MaxElementNum, Data);
				Serialization::Save(InOutOffset, ElementInputSize, Data);
				Serialization::Save(InOutOffset, ElementOutputSize, Data);
				Serialization::Save(InOutOffset, OutputEncodingSize, Data);
				Serialization::Save(InOutOffset, AttentionEncodingSize, Data);
				Serialization::Save(InOutOffset, AttentionHeadNum, Data);
				Serialization::Save(InOutOffset, SubLayer, Data);
				Serialization::Save(InOutOffset, QueryLayer, Data);
				Serialization::Save(InOutOffset, KeyLayer, Data);
				Serialization::Save(InOutOffset, ValueLayer, Data);
			}

			virtual void Evaluate(
				ILayerInstance* Instance,
				float* OutputBuffer,
				const float* InputBuffer,
				const uint32 BatchSize,
				const uint32 OutputBufferSize,
				const uint32 InputBufferSize,
				const uint32 OutputBufferStride,
				const uint32 InputBufferStride) override final
			{
				NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::FAggregateSetLayer::Evaluate);
				check(OutputBufferSize == GetOutputSize() && InputBufferSize == GetInputSize());
				check(OutputBufferStride >= GetOutputSize() && InputBufferStride >= GetInputSize());
				check(Instance && Instance->GetLayerType() == GetLayerType());

				FAggregateSetLayerInstance* AggregateSetInstance = StaticCast<FAggregateSetLayerInstance*>(Instance);

				OperatorNanCheck(InputBuffer, BatchSize, InputBufferSize, InputBufferStride);

				// Count the number of elements for each item in the batch
				
				OperatorAggregateCountElementNum(
					AggregateSetInstance->TotalElementNum,
					AggregateSetInstance->ElementNums.GetData(),
					AggregateSetInstance->ElementOffsets.GetData(),
					InputBuffer + MaxElementNum * ElementInputSize,
					BatchSize,
					MaxElementNum,
					InputBufferStride);

				// Gather Elements from all batches into one large tightly packed buffer

				OperatorAggregateGatherElements(
					AggregateSetInstance->InputElementBuffer.GetData(),
					InputBuffer,
					AggregateSetInstance->ElementNums.GetData(),
					AggregateSetInstance->ElementOffsets.GetData(),
					BatchSize,
					ElementInputSize,
					InputBufferStride);

				// Evaluate Sublayer on all elements

				SubLayer->Evaluate(
					AggregateSetInstance->SubLayerInstance.Get(),
					AggregateSetInstance->OutputElementBuffer.GetData(),
					AggregateSetInstance->InputElementBuffer.GetData(),
					AggregateSetInstance->TotalElementNum,
					ElementOutputSize,
					ElementInputSize,
					ElementOutputSize,
					ElementInputSize);

				// Compute Query on all elements

				QueryLayer->Evaluate(
					AggregateSetInstance->QueryInstance.Get(),
					AggregateSetInstance->QueryBuffer.GetData(),
					AggregateSetInstance->OutputElementBuffer.GetData(),
					AggregateSetInstance->TotalElementNum,
					AttentionHeadNum * AttentionEncodingSize,
					ElementOutputSize,
					AttentionHeadNum * AttentionEncodingSize,
					ElementOutputSize);

				// Compute Keys on all elements

				KeyLayer->Evaluate(
					AggregateSetInstance->KeyInstance.Get(),
					AggregateSetInstance->KeyBuffer.GetData(),
					AggregateSetInstance->OutputElementBuffer.GetData(),
					AggregateSetInstance->TotalElementNum,
					AttentionHeadNum * AttentionEncodingSize,
					ElementOutputSize,
					AttentionHeadNum * AttentionEncodingSize,
					ElementOutputSize);

				// Compute Values on all elements

				ValueLayer->Evaluate(
					AggregateSetInstance->ValueInstance.Get(),
					AggregateSetInstance->ValueBuffer.GetData(),
					AggregateSetInstance->OutputElementBuffer.GetData(),
					AggregateSetInstance->TotalElementNum,
					AttentionHeadNum * OutputEncodingSize,
					ElementOutputSize,
					AttentionHeadNum * OutputEncodingSize,
					ElementOutputSize);

				// Compute Attention

				OperatorAggregateDotProductAttention(
					AggregateSetInstance->AttentionBuffer.GetData(),
					AggregateSetInstance->QueryBuffer.GetData(),
					AggregateSetInstance->KeyBuffer.GetData(),
					AggregateSetInstance->TotalElementNum,
					AttentionEncodingSize,
					AttentionHeadNum);

				OperatorAggregateSoftmaxPlusOneInplace(
					AggregateSetInstance->AttentionMaxsBuffer.GetData(),
					AggregateSetInstance->AttentionDenomsBuffer.GetData(),
					AggregateSetInstance->AttentionBuffer.GetData(),
					AggregateSetInstance->ElementNums.GetData(),
					AggregateSetInstance->ElementOffsets.GetData(),
					BatchSize,
					AttentionHeadNum);

				OperatorAggregateAttentionSum(
					OutputBuffer,
					AggregateSetInstance->AttentionBuffer.GetData(),
					AggregateSetInstance->ValueBuffer.GetData(),
					AggregateSetInstance->ElementNums.GetData(),
					AggregateSetInstance->ElementOffsets.GetData(),
					BatchSize,
					OutputEncodingSize,
					AttentionHeadNum,
					OutputBufferStride);

				// Append Element Nums

				OperatorEncodeElementNums(
					OutputBuffer + AttentionHeadNum * OutputEncodingSize,
					AggregateSetInstance->ElementNums.GetData(),
					MaxElementNum,
					BatchSize,
					OutputBufferStride);

				OperatorNanCheck(OutputBuffer, BatchSize, OutputBufferSize, OutputBufferStride);
			}

			uint32 MaxElementNum = 0;
			uint32 OutputEncodingSize = 0;
			uint32 AttentionEncodingSize = 0;
			uint32 AttentionHeadNum = 0;

			TSharedPtr<ILayer> SubLayer;
			TSharedPtr<ILayer> QueryLayer;
			TSharedPtr<ILayer> KeyLayer;
			TSharedPtr<ILayer> ValueLayer;

			uint32 ElementInputSize = 0;
			uint32 ElementOutputSize = 0;
		};

		FAggregateSetLayerInstance::FAggregateSetLayerInstance(const FAggregateSetLayer& InAggregateSetLayer)
			: AggregateSetLayer(InAggregateSetLayer)
		{
			SubLayerInstance = AggregateSetLayer.SubLayer->MakeInstance();
			QueryInstance = AggregateSetLayer.QueryLayer->MakeInstance();
			KeyInstance = AggregateSetLayer.KeyLayer->MakeInstance();
			ValueInstance = AggregateSetLayer.ValueLayer->MakeInstance();
		}

		void FAggregateSetLayerInstance::SetMaxBatchSize(const uint32 MaxBatchSize)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::FAggregateSetLayerInstance::SetMaxBatchSize);

			if (SubLayerInstance) { SubLayerInstance->SetMaxBatchSize(MaxBatchSize * AggregateSetLayer.MaxElementNum); }
			if (QueryInstance) { QueryInstance->SetMaxBatchSize(MaxBatchSize * AggregateSetLayer.MaxElementNum); }
			if (KeyInstance) { KeyInstance->SetMaxBatchSize(MaxBatchSize * AggregateSetLayer.MaxElementNum); }
			if (ValueInstance) { ValueInstance->SetMaxBatchSize(MaxBatchSize * AggregateSetLayer.MaxElementNum); }

			ElementNums.SetNumUninitialized(MaxBatchSize, EAllowShrinking::No);
			ElementOffsets.SetNumUninitialized(MaxBatchSize, EAllowShrinking::No);
			
			InputElementBuffer.SetNumUninitialized(MaxBatchSize * AggregateSetLayer.MaxElementNum * AggregateSetLayer.ElementInputSize, EAllowShrinking::No);
			OutputElementBuffer.SetNumUninitialized(MaxBatchSize * AggregateSetLayer.MaxElementNum * AggregateSetLayer.ElementOutputSize, EAllowShrinking::No);
			QueryBuffer.SetNumUninitialized(MaxBatchSize * AggregateSetLayer.MaxElementNum * AggregateSetLayer.AttentionHeadNum * AggregateSetLayer.AttentionEncodingSize, EAllowShrinking::No);
			KeyBuffer.SetNumUninitialized(MaxBatchSize * AggregateSetLayer.MaxElementNum * AggregateSetLayer.AttentionHeadNum * AggregateSetLayer.AttentionEncodingSize, EAllowShrinking::No);
			ValueBuffer.SetNumUninitialized(MaxBatchSize * AggregateSetLayer.MaxElementNum * AggregateSetLayer.AttentionHeadNum * AggregateSetLayer.OutputEncodingSize, EAllowShrinking::No);
			AttentionMaxsBuffer.SetNumUninitialized(MaxBatchSize * AggregateSetLayer.AttentionHeadNum, EAllowShrinking::No);
			AttentionDenomsBuffer.SetNumUninitialized(MaxBatchSize * AggregateSetLayer.AttentionHeadNum, EAllowShrinking::No);
			AttentionBuffer.SetNumUninitialized(MaxBatchSize * AggregateSetLayer.MaxElementNum * AggregateSetLayer.AttentionHeadNum, EAllowShrinking::No);
		}

		//--------------------------------------------------------------------------

		struct FAggregateOrExclusiveLayer;

		struct FAggregateOrExclusiveLayerInstance : public ILayerInstance
		{
			FAggregateOrExclusiveLayerInstance(const FAggregateOrExclusiveLayer& InAggregateOrExclusiveLayer);

			virtual void SetMaxBatchSize(const uint32 MaxBatchSize) override final;
			virtual ELayerType GetLayerType() const override final { return ELayerType::AggregateOrExclusive; }

			const FAggregateOrExclusiveLayer& AggregateOrExclusiveLayer;
			TArray<TSharedPtr<ILayerInstance>, TInlineAllocator<32>> SubLayerInstances;
			TArray<TSharedPtr<ILayerInstance>, TInlineAllocator<32>> EncoderInstances;

			TArray<TArray<uint32>, TInlineAllocator<32>> SubLayerBatchIndices;
			TArray<TArray<float>, TInlineAllocator<32>> SubLayerInputBuffers;
			TArray<TArray<float>, TInlineAllocator<32>> SubLayerOutputBuffers;
			TArray<TArray<float>, TInlineAllocator<32>> SubLayerEncodingBuffers;
		};

		struct FAggregateOrExclusiveLayer : public ILayer
		{
			virtual TSharedPtr<ILayerInstance> MakeInstance() const { return MakeShared<FAggregateOrExclusiveLayerInstance>(*this); };
			virtual ELayerType GetLayerType() const override final { return ELayerType::AggregateOrExclusive; }
			virtual uint32 GetInputSize() const override final { return MaxSubLayerInputSize + SubLayers.Num(); }
			virtual uint32 GetOutputSize() const override final { return OutputEncodingSize + SubLayers.Num(); }

			virtual void SerializationSize(uint64& InOutOffset) const override final
			{
				Serialization::Size(InOutOffset, (uint32)SubLayers.Num());
				Serialization::Size(InOutOffset, OutputEncodingSize);
				Serialization::Size(InOutOffset, SubLayerInputSizes);
				Serialization::Size(InOutOffset, SubLayerOutputSizes);
				Serialization::Size(InOutOffset, SubLayers);
				Serialization::Size(InOutOffset, Encoders);
			}

			virtual void SerializationLoad(uint64& InOutOffset, TConstArrayView<uint8> Data) override final
			{
				uint32 SubLayerNum = 0;
				Serialization::Load(InOutOffset, SubLayerNum, Data);
				Serialization::Load(InOutOffset, OutputEncodingSize, Data);
				Serialization::Load(InOutOffset, SubLayerInputSizes, Data, SubLayerNum);
				Serialization::Load(InOutOffset, SubLayerOutputSizes, Data, SubLayerNum);
				SubLayers.Init(nullptr, SubLayerNum);
				Serialization::Load(InOutOffset, SubLayers, Data);
				Encoders.Init(nullptr, SubLayerNum);
				Serialization::Load(InOutOffset, Encoders, Data);
				PostLoad();
			}

			void PostLoad()
			{
				const uint32 SubLayerNum = SubLayers.Num();
				MaxSubLayerInputSize = 0;
				for (uint32 SubLayerIdx = 0; SubLayerIdx < SubLayerNum; SubLayerIdx++)
				{
					MaxSubLayerInputSize = FMath::Max(MaxSubLayerInputSize, SubLayerInputSizes[SubLayerIdx]);
				}
			}

			virtual void SerializationSave(uint64& InOutOffset, TArrayView<uint8> Data) const override final
			{
				Serialization::Save(InOutOffset, (uint32)SubLayers.Num(), Data);
				Serialization::Save(InOutOffset, OutputEncodingSize, Data);
				Serialization::Save(InOutOffset, SubLayerInputSizes, Data);
				Serialization::Save(InOutOffset, SubLayerOutputSizes, Data);
				Serialization::Save(InOutOffset, SubLayers, Data);
				Serialization::Save(InOutOffset, Encoders, Data);
			}

			virtual void Evaluate(
				ILayerInstance* Instance,
				float* OutputBuffer,
				const float* InputBuffer,
				const uint32 BatchSize,
				const uint32 OutputBufferSize,
				const uint32 InputBufferSize,
				const uint32 OutputBufferStride,
				const uint32 InputBufferStride) override final
			{
				NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::FAggregateOrExclusiveLayer::Evaluate);
				check(OutputBufferSize == GetOutputSize() && InputBufferSize == GetInputSize());
				check(OutputBufferStride >= GetOutputSize() && InputBufferStride >= GetInputSize());
				check(Instance && Instance->GetLayerType() == GetLayerType());

				FAggregateOrExclusiveLayerInstance* AggregateOrExclusiveInstance = StaticCast<FAggregateOrExclusiveLayerInstance*>(Instance);

				OperatorNanCheck(InputBuffer, BatchSize, InputBufferSize, InputBufferStride);

				const uint32 SubLayerNum = SubLayers.Num();

				// Gather the batch indices used by each sub-layer

				OperatorGatherSubLayerBatchIndicesExclusive(
					AggregateOrExclusiveInstance->SubLayerBatchIndices,
					InputBuffer + MaxSubLayerInputSize,
					BatchSize,
					SubLayerNum,
					InputBufferStride);

				// Evaluate Sublayers

				for (uint32 SubLayerIdx = 0; SubLayerIdx < SubLayerNum; SubLayerIdx++)
				{
					const uint32 SubLayerBatchSize = AggregateOrExclusiveInstance->SubLayerBatchIndices[SubLayerIdx].Num();

					if (SubLayerBatchSize == 0) { continue; }

					OperatorGather(
						AggregateOrExclusiveInstance->SubLayerInputBuffers[SubLayerIdx].GetData(),
						InputBuffer,
						AggregateOrExclusiveInstance->SubLayerBatchIndices[SubLayerIdx].GetData(),
						SubLayerBatchSize,
						SubLayerInputSizes[SubLayerIdx],
						SubLayerInputSizes[SubLayerIdx],
						InputBufferStride);

					SubLayers[SubLayerIdx]->Evaluate(
						AggregateOrExclusiveInstance->SubLayerInstances[SubLayerIdx].Get(),
						AggregateOrExclusiveInstance->SubLayerOutputBuffers[SubLayerIdx].GetData(),
						AggregateOrExclusiveInstance->SubLayerInputBuffers[SubLayerIdx].GetData(),
						SubLayerBatchSize,
						SubLayerOutputSizes[SubLayerIdx],
						SubLayerInputSizes[SubLayerIdx],
						SubLayerOutputSizes[SubLayerIdx],
						SubLayerInputSizes[SubLayerIdx]);

					Encoders[SubLayerIdx]->Evaluate(
						AggregateOrExclusiveInstance->EncoderInstances[SubLayerIdx].Get(),
						AggregateOrExclusiveInstance->SubLayerEncodingBuffers[SubLayerIdx].GetData(),
						AggregateOrExclusiveInstance->SubLayerOutputBuffers[SubLayerIdx].GetData(),
						SubLayerBatchSize,
						OutputEncodingSize,
						SubLayerOutputSizes[SubLayerIdx],
						OutputEncodingSize,
						SubLayerOutputSizes[SubLayerIdx]);

					OperatorScatter(
						OutputBuffer,
						AggregateOrExclusiveInstance->SubLayerEncodingBuffers[SubLayerIdx].GetData(),
						AggregateOrExclusiveInstance->SubLayerBatchIndices[SubLayerIdx].GetData(),
						SubLayerBatchSize,
						OutputEncodingSize,
						OutputBufferStride,
						OutputEncodingSize);
				}

				// Append SubLayer Mask

				OperatorCopy(
					OutputBuffer + OutputEncodingSize,
					InputBuffer + MaxSubLayerInputSize,
					BatchSize,
					SubLayerNum,
					OutputBufferStride,
					InputBufferStride);

				OperatorNanCheck(OutputBuffer, BatchSize, OutputBufferSize, OutputBufferStride);
			}

			uint32 OutputEncodingSize = 0;
			TConstArrayView<uint32> SubLayerInputSizes;
			TConstArrayView<uint32> SubLayerOutputSizes;
			TArray<TSharedPtr<ILayer>, TInlineAllocator<32>> SubLayers;
			TArray<TSharedPtr<ILayer>, TInlineAllocator<32>> Encoders;

			uint32 MaxSubLayerInputSize = 0;
		};

		FAggregateOrExclusiveLayerInstance::FAggregateOrExclusiveLayerInstance(const FAggregateOrExclusiveLayer& InAggregateOrExclusiveLayer)
			: AggregateOrExclusiveLayer(InAggregateOrExclusiveLayer)
		{
			const uint32 SubLayerNum = AggregateOrExclusiveLayer.SubLayers.Num();

			SubLayerInstances.Init(nullptr, SubLayerNum);
			EncoderInstances.Init(nullptr, SubLayerNum);

			SubLayerBatchIndices.SetNum(SubLayerNum);
			SubLayerInputBuffers.SetNum(SubLayerNum);
			SubLayerOutputBuffers.SetNum(SubLayerNum);
			SubLayerEncodingBuffers.SetNum(SubLayerNum);

			for (uint32 SubLayerIdx = 0; SubLayerIdx < SubLayerNum; SubLayerIdx++)
			{
				SubLayerInstances[SubLayerIdx] = AggregateOrExclusiveLayer.SubLayers[SubLayerIdx]->MakeInstance();
				EncoderInstances[SubLayerIdx] = AggregateOrExclusiveLayer.Encoders[SubLayerIdx]->MakeInstance();
			}
		}

		void FAggregateOrExclusiveLayerInstance::SetMaxBatchSize(const uint32 MaxBatchSize)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::FAggregateOrExclusiveLayerInstance::SetMaxBatchSize);

			const uint32 SubLayerNum = AggregateOrExclusiveLayer.SubLayers.Num();

			for (uint32 SubLayerIdx = 0; SubLayerIdx < SubLayerNum; SubLayerIdx++)
			{
				if (SubLayerInstances[SubLayerIdx]) { SubLayerInstances[SubLayerIdx]->SetMaxBatchSize(MaxBatchSize); }
				if (EncoderInstances[SubLayerIdx]) { EncoderInstances[SubLayerIdx]->SetMaxBatchSize(MaxBatchSize); }

				SubLayerBatchIndices[SubLayerIdx].Empty(MaxBatchSize);
				SubLayerInputBuffers[SubLayerIdx].SetNumUninitialized(MaxBatchSize * AggregateOrExclusiveLayer.SubLayerInputSizes[SubLayerIdx], EAllowShrinking::No);
				SubLayerOutputBuffers[SubLayerIdx].SetNumUninitialized(MaxBatchSize * AggregateOrExclusiveLayer.SubLayerOutputSizes[SubLayerIdx], EAllowShrinking::No);
				SubLayerEncodingBuffers[SubLayerIdx].SetNumUninitialized(MaxBatchSize * AggregateOrExclusiveLayer.OutputEncodingSize, EAllowShrinking::No);
			}
		}

		//--------------------------------------------------------------------------

		struct FAggregateOrInclusiveLayer;

		struct FAggregateOrInclusiveLayerInstance : public ILayerInstance
		{
			FAggregateOrInclusiveLayerInstance(const FAggregateOrInclusiveLayer& InAggregateOrInclusiveLayer);

			virtual void SetMaxBatchSize(const uint32 MaxBatchSize) override final;
			virtual ELayerType GetLayerType() const override final { return ELayerType::AggregateOrInclusive; }

			const FAggregateOrInclusiveLayer& AggregateOrInclusiveLayer;
			TArray<TSharedPtr<ILayerInstance>, TInlineAllocator<32>> SubLayerInstances;
			TArray<TSharedPtr<ILayerInstance>, TInlineAllocator<32>> QueryInstances;
			TArray<TSharedPtr<ILayerInstance>, TInlineAllocator<32>> KeyInstances;
			TArray<TSharedPtr<ILayerInstance>, TInlineAllocator<32>> ValueInstances;

			TArray<TArray<uint32>, TInlineAllocator<32>> SubLayerBatchIndices;
			TArray<TArray<float>, TInlineAllocator<32>> SubLayerInputBuffers;
			TArray<TArray<float>, TInlineAllocator<32>> SubLayerOutputBuffers;
			TArray<TArray<float>, TInlineAllocator<32>> SubLayerQueryBuffers;
			TArray<TArray<float>, TInlineAllocator<32>> SubLayerKeyBuffers;
			TArray<TArray<float>, TInlineAllocator<32>> SubLayerValueBuffers;

			uint32 TotalElementNum = 0;
			TArray<uint32, TInlineAllocator<32>> ElementAccum;
			TArray<uint32, TInlineAllocator<32>> ElementNums;
			TArray<uint32, TInlineAllocator<32>> ElementOffsets;

			TArray<float> AttentionMaxsBuffer;
			TArray<float> AttentionDenomsBuffer;
			TArray<float> AttentionBuffer;
			TArray<float> QueryBuffer;
			TArray<float> KeyBuffer;
			TArray<float> ValueBuffer;
		};

		struct FAggregateOrInclusiveLayer : public ILayer
		{
			virtual TSharedPtr<ILayerInstance> MakeInstance() const { return MakeShared<FAggregateOrInclusiveLayerInstance>(*this); };
			virtual ELayerType GetLayerType() const override final { return ELayerType::AggregateOrInclusive; }
			virtual uint32 GetInputSize() const override final { return TotalSubLayerInputSize + SubLayers.Num(); }
			virtual uint32 GetOutputSize() const override final { return AttentionHeadNum * OutputEncodingSize + SubLayers.Num(); }

			virtual void SerializationSize(uint64& InOutOffset) const override final
			{
				Serialization::Size(InOutOffset, (uint32)SubLayers.Num());
				Serialization::Size(InOutOffset, OutputEncodingSize);
				Serialization::Size(InOutOffset, AttentionEncodingSize);
				Serialization::Size(InOutOffset, AttentionHeadNum);
				Serialization::Size(InOutOffset, SubLayerInputSizes);
				Serialization::Size(InOutOffset, SubLayerOutputSizes);
				Serialization::Size(InOutOffset, SubLayers);
				Serialization::Size(InOutOffset, QueryLayers);
				Serialization::Size(InOutOffset, KeyLayers);
				Serialization::Size(InOutOffset, ValueLayers);
			}

			virtual void SerializationLoad(uint64& InOutOffset, TConstArrayView<uint8> Data) override final
			{
				uint32 SubLayerNum = 0;
				Serialization::Load(InOutOffset, SubLayerNum, Data);
				Serialization::Load(InOutOffset, OutputEncodingSize, Data);
				Serialization::Load(InOutOffset, AttentionEncodingSize, Data);
				Serialization::Load(InOutOffset, AttentionHeadNum, Data);
				Serialization::Load(InOutOffset, SubLayerInputSizes, Data, SubLayerNum);
				Serialization::Load(InOutOffset, SubLayerOutputSizes, Data, SubLayerNum);
				SubLayers.Init(nullptr, SubLayerNum);
				Serialization::Load(InOutOffset, SubLayers, Data);
				QueryLayers.Init(nullptr, SubLayerNum);
				Serialization::Load(InOutOffset, QueryLayers, Data);
				KeyLayers.Init(nullptr, SubLayerNum);
				Serialization::Load(InOutOffset, KeyLayers, Data);
				ValueLayers.Init(nullptr, SubLayerNum);
				Serialization::Load(InOutOffset, ValueLayers, Data);
				PostLoad();
			}

			void PostLoad()
			{
				const uint32 SubLayerNum = SubLayers.Num();
				TotalSubLayerInputSize = 0;
				SubLayerInputOffsets.SetNumUninitialized(SubLayerNum);
				for (uint32 SubLayerIdx = 0; SubLayerIdx < SubLayerNum; SubLayerIdx++)
				{
					SubLayerInputOffsets[SubLayerIdx] = TotalSubLayerInputSize;
					TotalSubLayerInputSize += SubLayerInputSizes[SubLayerIdx];
				}
			}

			virtual void SerializationSave(uint64& InOutOffset, TArrayView<uint8> Data) const override final
			{
				Serialization::Save(InOutOffset, (uint32)SubLayers.Num(), Data);
				Serialization::Save(InOutOffset, OutputEncodingSize, Data);
				Serialization::Save(InOutOffset, AttentionEncodingSize, Data);
				Serialization::Save(InOutOffset, AttentionHeadNum, Data);
				Serialization::Save(InOutOffset, SubLayerInputSizes, Data);
				Serialization::Save(InOutOffset, SubLayerOutputSizes, Data);
				Serialization::Save(InOutOffset, SubLayers, Data);
				Serialization::Save(InOutOffset, QueryLayers, Data);
				Serialization::Save(InOutOffset, KeyLayers, Data);
				Serialization::Save(InOutOffset, ValueLayers, Data);
			}

			virtual void Evaluate(
				ILayerInstance* Instance,
				float* OutputBuffer,
				const float* InputBuffer,
				const uint32 BatchSize,
				const uint32 OutputBufferSize,
				const uint32 InputBufferSize,
				const uint32 OutputBufferStride,
				const uint32 InputBufferStride) override final
			{
				NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::FAggregateOrInclusiveLayer::Evaluate);
				check(OutputBufferSize == GetOutputSize() && InputBufferSize == GetInputSize());
				check(OutputBufferStride >= GetOutputSize() && InputBufferStride >= GetInputSize());
				check(Instance && Instance->GetLayerType() == GetLayerType());

				FAggregateOrInclusiveLayerInstance* AggregateOrInclusiveInstance = StaticCast<FAggregateOrInclusiveLayerInstance*>(Instance);

				OperatorNanCheck(InputBuffer, BatchSize, InputBufferSize, InputBufferStride);

				const uint32 SubLayerNum = SubLayers.Num();

				// Count the number of sub-layer used by each item in the batch

				OperatorAggregateCountElementNum(
					AggregateOrInclusiveInstance->TotalElementNum,
					AggregateOrInclusiveInstance->ElementNums.GetData(),
					AggregateOrInclusiveInstance->ElementOffsets.GetData(),
					InputBuffer + TotalSubLayerInputSize,
					BatchSize,
					SubLayerNum,
					InputBufferStride);

				// Gather the batch indices used by each sub-layer

				OperatorGatherSubLayerBatchIndicesInclusive(
					AggregateOrInclusiveInstance->SubLayerBatchIndices,
					InputBuffer + TotalSubLayerInputSize,
					BatchSize,
					SubLayerNum,
					InputBufferStride);

				// Evaluate Each Sublayer on the associated batch items

				for (uint32 SubLayerIdx = 0; SubLayerIdx < SubLayerNum; SubLayerIdx++)
				{
					const uint32 SubLayerBatchSize = AggregateOrInclusiveInstance->SubLayerBatchIndices[SubLayerIdx].Num();

					if (SubLayerBatchSize == 0) { continue; }

					OperatorGather(
						AggregateOrInclusiveInstance->SubLayerInputBuffers[SubLayerIdx].GetData(),
						InputBuffer + SubLayerInputOffsets[SubLayerIdx],
						AggregateOrInclusiveInstance->SubLayerBatchIndices[SubLayerIdx].GetData(),
						SubLayerBatchSize,
						SubLayerInputSizes[SubLayerIdx],
						SubLayerInputSizes[SubLayerIdx],
						InputBufferStride);

					SubLayers[SubLayerIdx]->Evaluate(
						AggregateOrInclusiveInstance->SubLayerInstances[SubLayerIdx].Get(),
						AggregateOrInclusiveInstance->SubLayerOutputBuffers[SubLayerIdx].GetData(),
						AggregateOrInclusiveInstance->SubLayerInputBuffers[SubLayerIdx].GetData(),
						SubLayerBatchSize,
						SubLayerOutputSizes[SubLayerIdx],
						SubLayerInputSizes[SubLayerIdx],
						SubLayerOutputSizes[SubLayerIdx],
						SubLayerInputSizes[SubLayerIdx]);

					QueryLayers[SubLayerIdx]->Evaluate(
						AggregateOrInclusiveInstance->QueryInstances[SubLayerIdx].Get(),
						AggregateOrInclusiveInstance->SubLayerQueryBuffers[SubLayerIdx].GetData(),
						AggregateOrInclusiveInstance->SubLayerOutputBuffers[SubLayerIdx].GetData(),
						SubLayerBatchSize,
						AttentionHeadNum * AttentionEncodingSize,
						SubLayerOutputSizes[SubLayerIdx],
						AttentionHeadNum * AttentionEncodingSize,
						SubLayerOutputSizes[SubLayerIdx]);

					KeyLayers[SubLayerIdx]->Evaluate(
						AggregateOrInclusiveInstance->KeyInstances[SubLayerIdx].Get(),
						AggregateOrInclusiveInstance->SubLayerKeyBuffers[SubLayerIdx].GetData(),
						AggregateOrInclusiveInstance->SubLayerOutputBuffers[SubLayerIdx].GetData(),
						SubLayerBatchSize,
						AttentionHeadNum * AttentionEncodingSize,
						SubLayerOutputSizes[SubLayerIdx],
						AttentionHeadNum * AttentionEncodingSize,
						SubLayerOutputSizes[SubLayerIdx]);

					ValueLayers[SubLayerIdx]->Evaluate(
						AggregateOrInclusiveInstance->ValueInstances[SubLayerIdx].Get(),
						AggregateOrInclusiveInstance->SubLayerValueBuffers[SubLayerIdx].GetData(),
						AggregateOrInclusiveInstance->SubLayerOutputBuffers[SubLayerIdx].GetData(),
						SubLayerBatchSize,
						AttentionHeadNum * OutputEncodingSize,
						SubLayerOutputSizes[SubLayerIdx],
						AttentionHeadNum * OutputEncodingSize,
						SubLayerOutputSizes[SubLayerIdx]);
				}

				// Gather queries, keys, and values from sub-layers into tightly packed element lists
				// which we can attend over using the ElementNums and ElementOffsets arrays

				OperatorAggregateGatherFromSubLayers(
					AggregateOrInclusiveInstance->QueryBuffer.GetData(),
					AggregateOrInclusiveInstance->KeyBuffer.GetData(),
					AggregateOrInclusiveInstance->ValueBuffer.GetData(),
					AggregateOrInclusiveInstance->ElementAccum.GetData(),
					AggregateOrInclusiveInstance->ElementNums.GetData(),
					AggregateOrInclusiveInstance->ElementOffsets.GetData(),
					AggregateOrInclusiveInstance->SubLayerBatchIndices,
					AggregateOrInclusiveInstance->SubLayerQueryBuffers,
					AggregateOrInclusiveInstance->SubLayerKeyBuffers,
					AggregateOrInclusiveInstance->SubLayerValueBuffers,
					BatchSize,
					AttentionHeadNum * AttentionEncodingSize,
					AttentionHeadNum * AttentionEncodingSize,
					AttentionHeadNum * OutputEncodingSize);

				// Compute Attention

				OperatorAggregateDotProductAttention(
					AggregateOrInclusiveInstance->AttentionBuffer.GetData(),
					AggregateOrInclusiveInstance->QueryBuffer.GetData(),
					AggregateOrInclusiveInstance->KeyBuffer.GetData(),
					AggregateOrInclusiveInstance->TotalElementNum,
					AttentionEncodingSize,
					AttentionHeadNum);

				OperatorAggregateSoftmaxPlusOneInplace(
					AggregateOrInclusiveInstance->AttentionMaxsBuffer.GetData(),
					AggregateOrInclusiveInstance->AttentionDenomsBuffer.GetData(),
					AggregateOrInclusiveInstance->AttentionBuffer.GetData(),
					AggregateOrInclusiveInstance->ElementNums.GetData(),
					AggregateOrInclusiveInstance->ElementOffsets.GetData(),
					BatchSize,
					AttentionHeadNum);

				OperatorAggregateAttentionSum(
					OutputBuffer,
					AggregateOrInclusiveInstance->AttentionBuffer.GetData(),
					AggregateOrInclusiveInstance->ValueBuffer.GetData(),
					AggregateOrInclusiveInstance->ElementNums.GetData(),
					AggregateOrInclusiveInstance->ElementOffsets.GetData(),
					BatchSize,
					OutputEncodingSize,
					AttentionHeadNum,
					OutputBufferStride);

				// Append Element Mask

				OperatorCopy(
					OutputBuffer + AttentionHeadNum * OutputEncodingSize,
					InputBuffer + TotalSubLayerInputSize,
					BatchSize,
					SubLayerNum,
					OutputBufferStride,
					InputBufferStride);

				OperatorNanCheck(OutputBuffer, BatchSize, OutputBufferSize, OutputBufferStride);
			}

			uint32 OutputEncodingSize = 0;
			uint32 AttentionEncodingSize = 0;
			uint32 AttentionHeadNum = 0;
			TConstArrayView<uint32> SubLayerInputSizes;
			TConstArrayView<uint32> SubLayerOutputSizes;
			TArray<TSharedPtr<ILayer>, TInlineAllocator<32>> SubLayers;
			TArray<TSharedPtr<ILayer>, TInlineAllocator<32>> QueryLayers;
			TArray<TSharedPtr<ILayer>, TInlineAllocator<32>> KeyLayers;
			TArray<TSharedPtr<ILayer>, TInlineAllocator<32>> ValueLayers;

			uint32 TotalSubLayerInputSize = 0;
			TArray<uint32, TInlineAllocator<32>> SubLayerInputOffsets;
		};

		FAggregateOrInclusiveLayerInstance::FAggregateOrInclusiveLayerInstance(const FAggregateOrInclusiveLayer& InAggregateOrInclusiveLayer)
			: AggregateOrInclusiveLayer(InAggregateOrInclusiveLayer)
		{
			const uint32 SubLayerNum = AggregateOrInclusiveLayer.SubLayers.Num();

			SubLayerInstances.Init(nullptr, SubLayerNum);
			QueryInstances.Init(nullptr, SubLayerNum);
			KeyInstances.Init(nullptr, SubLayerNum);
			ValueInstances.Init(nullptr, SubLayerNum);

			SubLayerBatchIndices.SetNum(SubLayerNum);
			SubLayerInputBuffers.SetNum(SubLayerNum);
			SubLayerOutputBuffers.SetNum(SubLayerNum);
			SubLayerQueryBuffers.SetNum(SubLayerNum);
			SubLayerKeyBuffers.SetNum(SubLayerNum);
			SubLayerValueBuffers.SetNum(SubLayerNum);

			for (uint32 SubLayerIdx = 0; SubLayerIdx < SubLayerNum; SubLayerIdx++)
			{
				SubLayerInstances[SubLayerIdx] = AggregateOrInclusiveLayer.SubLayers[SubLayerIdx]->MakeInstance();
				QueryInstances[SubLayerIdx] = AggregateOrInclusiveLayer.QueryLayers[SubLayerIdx]->MakeInstance();
				KeyInstances[SubLayerIdx] = AggregateOrInclusiveLayer.KeyLayers[SubLayerIdx]->MakeInstance();
				ValueInstances[SubLayerIdx] = AggregateOrInclusiveLayer.ValueLayers[SubLayerIdx]->MakeInstance();
			}
		}

		void FAggregateOrInclusiveLayerInstance::SetMaxBatchSize(const uint32 MaxBatchSize)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::FAggregateOrInclusiveLayerInstance::SetMaxBatchSize);

			const uint32 SubLayerNum = AggregateOrInclusiveLayer.SubLayers.Num();

			for (uint32 SubLayerIdx = 0; SubLayerIdx < SubLayerNum; SubLayerIdx++)
			{
				if (SubLayerInstances[SubLayerIdx]) { SubLayerInstances[SubLayerIdx]->SetMaxBatchSize(MaxBatchSize); }
				if (QueryInstances[SubLayerIdx]) { QueryInstances[SubLayerIdx]->SetMaxBatchSize(MaxBatchSize); }
				if (KeyInstances[SubLayerIdx]) { KeyInstances[SubLayerIdx]->SetMaxBatchSize(MaxBatchSize); }
				if (ValueInstances[SubLayerIdx]) { ValueInstances[SubLayerIdx]->SetMaxBatchSize(MaxBatchSize); }

				SubLayerBatchIndices[SubLayerIdx].Empty(MaxBatchSize);
				SubLayerInputBuffers[SubLayerIdx].SetNumUninitialized(MaxBatchSize * AggregateOrInclusiveLayer.SubLayerInputSizes[SubLayerIdx], EAllowShrinking::No);
				SubLayerOutputBuffers[SubLayerIdx].SetNumUninitialized(MaxBatchSize * AggregateOrInclusiveLayer.SubLayerOutputSizes[SubLayerIdx], EAllowShrinking::No);
				SubLayerQueryBuffers[SubLayerIdx].SetNumUninitialized(MaxBatchSize * AggregateOrInclusiveLayer.AttentionHeadNum * AggregateOrInclusiveLayer.AttentionEncodingSize, EAllowShrinking::No);
				SubLayerKeyBuffers[SubLayerIdx].SetNumUninitialized(MaxBatchSize * AggregateOrInclusiveLayer.AttentionHeadNum * AggregateOrInclusiveLayer.AttentionEncodingSize, EAllowShrinking::No);
				SubLayerValueBuffers[SubLayerIdx].SetNumUninitialized(MaxBatchSize * AggregateOrInclusiveLayer.AttentionHeadNum * AggregateOrInclusiveLayer.OutputEncodingSize, EAllowShrinking::No);
			}

			TotalElementNum = 0;
			ElementAccum.SetNumUninitialized(MaxBatchSize, EAllowShrinking::No);
			ElementNums.SetNumUninitialized(MaxBatchSize, EAllowShrinking::No);
			ElementOffsets.SetNumUninitialized(MaxBatchSize, EAllowShrinking::No);

			AttentionMaxsBuffer.SetNumUninitialized(MaxBatchSize * AggregateOrInclusiveLayer.AttentionHeadNum, EAllowShrinking::No);
			AttentionDenomsBuffer.SetNumUninitialized(MaxBatchSize * AggregateOrInclusiveLayer.AttentionHeadNum, EAllowShrinking::No);
			AttentionBuffer.SetNumUninitialized(MaxBatchSize * SubLayerNum * AggregateOrInclusiveLayer.AttentionHeadNum, EAllowShrinking::No);
			QueryBuffer.SetNumUninitialized(MaxBatchSize * SubLayerNum * AggregateOrInclusiveLayer.AttentionHeadNum * AggregateOrInclusiveLayer.AttentionEncodingSize);
			KeyBuffer.SetNumUninitialized(MaxBatchSize * SubLayerNum * AggregateOrInclusiveLayer.AttentionHeadNum * AggregateOrInclusiveLayer.AttentionEncodingSize);
			ValueBuffer.SetNumUninitialized(MaxBatchSize * SubLayerNum * AggregateOrInclusiveLayer.AttentionHeadNum * AggregateOrInclusiveLayer.OutputEncodingSize);
		}


		//--------------------------------------------------------------------------

		struct FClampLayer : public ILayer
		{
			virtual ELayerType GetLayerType() const override final { return ELayerType::Clamp; }
			virtual uint32 GetInputSize() const override final { return InputOutputSize; }
			virtual uint32 GetOutputSize() const override final { return InputOutputSize; }

			virtual void SerializationSize(uint64& InOutOffset) const override final
			{
				Serialization::Size(InOutOffset, InputOutputSize);
				Serialization::Size(InOutOffset, MinValues);
				Serialization::Size(InOutOffset, MaxValues);
			}

			virtual void SerializationLoad(uint64& InOutOffset, TConstArrayView<uint8> Data) override final
			{
				Serialization::Load(InOutOffset, InputOutputSize, Data);
				Serialization::Load(InOutOffset, MinValues, Data, InputOutputSize);
				Serialization::Load(InOutOffset, MaxValues, Data, InputOutputSize);
			}

			virtual void SerializationSave(uint64& InOutOffset, TArrayView<uint8> Data) const override final
			{
				Serialization::Save(InOutOffset, InputOutputSize, Data);
				Serialization::Save(InOutOffset, MinValues, Data);
				Serialization::Save(InOutOffset, MaxValues, Data);
			}

			virtual void Evaluate(
				ILayerInstance* Instance,
				float* OutputBuffer,
				const float* InputBuffer,
				const uint32 BatchSize,
				const uint32 OutputBufferSize,
				const uint32 InputBufferSize,
				const uint32 OutputBufferStride,
				const uint32 InputBufferStride) override final
			{
				NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::FClampLayer::Evaluate);
				check(OutputBufferSize == GetOutputSize() && InputBufferSize == GetInputSize());
				check(OutputBufferStride >= GetOutputSize() && InputBufferStride >= GetInputSize());
				check(Instance == nullptr);
				OperatorNanCheck(InputBuffer, BatchSize, InputBufferSize, InputBufferStride);

				OperatorClamp(
					OutputBuffer,
					InputBuffer,
					MinValues.GetData(),
					MaxValues.GetData(),
					BatchSize,
					InputOutputSize,
					OutputBufferStride,
					InputBufferStride);

				OperatorNanCheck(OutputBuffer, BatchSize, OutputBufferSize, OutputBufferStride);
			}

			uint32 InputOutputSize = 0;
			TConstArrayView<float> MinValues;
			TConstArrayView<float> MaxValues;
		};

		//--------------------------------------------------------------------------

		struct FSparseMixtureOfExpertsLayer;

		struct FSparseMixtureOfExpertsLayerInstance : public ILayerInstance
		{
			FSparseMixtureOfExpertsLayerInstance(const FSparseMixtureOfExpertsLayer& InSparseMixtureOfExpertsLayer);

			virtual void SetMaxBatchSize(const uint32 MaxBatchSize) override final;
			virtual ELayerType GetLayerType() const override final { return ELayerType::SparseMixtureOfExperts; }

			const FSparseMixtureOfExpertsLayer& SparseMixtureOfExpertsLayer;
			TSharedPtr<ILayerInstance> GatingInstance;
			TArray<float> GatingOutputBuffer;

			TArray<TSharedPtr<ILayerInstance>, TInlineAllocator<32>> SubLayerInstances;
			TArray<uint32> BatchSubLayerIndex0;
			TArray<uint32> BatchSubLayerIndex1;
			TArray<float> BatchSubLayerWeight0;
			TArray<float> BatchSubLayerWeight1;
			TArray<uint32> BatchSubLayerOutputIndex0;
			TArray<uint32> BatchSubLayerOutputIndex1;
			TArray<TArray<uint32>, TInlineAllocator<32>> SubLayerBatchIndices;
			TArray<TArray<float>, TInlineAllocator<32>> SubLayerInputBuffers;
			TArray<TArray<float>, TInlineAllocator<32>> SubLayerOutputBuffers;
		};

		struct FSparseMixtureOfExpertsLayer : public ILayer
		{
			virtual TSharedPtr<ILayerInstance> MakeInstance() const { return MakeShared<FSparseMixtureOfExpertsLayerInstance>(*this); };
			virtual ELayerType GetLayerType() const override final { return ELayerType::SparseMixtureOfExperts; }
			virtual uint32 GetInputSize() const override final { return InputSize; }
			virtual uint32 GetOutputSize() const override final { return OutputSize; }

			virtual void SerializationSize(uint64& InOutOffset) const override final
			{
				Serialization::Size(InOutOffset, InputSize);
				Serialization::Size(InOutOffset, OutputSize);
				Serialization::Size(InOutOffset, GatingLayer);
				Serialization::Size(InOutOffset, (uint32)SubLayers.Num());
				Serialization::Size(InOutOffset, SubLayers);
			}

			virtual void SerializationLoad(uint64& InOutOffset, TConstArrayView<uint8> Data) override final
			{
				Serialization::Load(InOutOffset, InputSize, Data);
				Serialization::Load(InOutOffset, OutputSize, Data);
				Serialization::Load(InOutOffset, GatingLayer, Data);
				uint32 SubLayerNum = 0;
				Serialization::Load(InOutOffset, SubLayerNum, Data);
				SubLayers.Init(nullptr, SubLayerNum);
				Serialization::Load(InOutOffset, SubLayers, Data);
			}

			virtual void SerializationSave(uint64& InOutOffset, TArrayView<uint8> Data) const override final
			{
				Serialization::Save(InOutOffset, InputSize, Data);
				Serialization::Save(InOutOffset, OutputSize, Data);
				Serialization::Save(InOutOffset, GatingLayer, Data);
				Serialization::Save(InOutOffset, (uint32)SubLayers.Num(), Data);
				Serialization::Save(InOutOffset, SubLayers, Data);
			}

			virtual void Evaluate(
				ILayerInstance* Instance,
				float* OutputBuffer,
				const float* InputBuffer,
				const uint32 BatchSize,
				const uint32 OutputBufferSize,
				const uint32 InputBufferSize,
				const uint32 OutputBufferStride,
				const uint32 InputBufferStride) override final
			{
				NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::FSparseMixtureOfExpertsLayer::Evaluate);
				check(OutputBufferSize == GetOutputSize() && InputBufferSize == GetInputSize());
				check(OutputBufferStride >= GetOutputSize() && InputBufferStride >= GetInputSize());
				check(Instance && Instance->GetLayerType() == GetLayerType());

				FSparseMixtureOfExpertsLayerInstance* SparseMixtureOfExpertsInstance = StaticCast<FSparseMixtureOfExpertsLayerInstance*>(Instance);

				OperatorNanCheck(InputBuffer, BatchSize, InputBufferSize, InputBufferStride);

				const uint32 SubLayerNum = SubLayers.Num();

				// Evaluate Gating Layer

				GatingLayer->Evaluate(
					SparseMixtureOfExpertsInstance->GatingInstance.Get(),
					SparseMixtureOfExpertsInstance->GatingOutputBuffer.GetData(),
					InputBuffer,
					BatchSize,
					SubLayerNum,
					InputBufferSize,
					SubLayerNum,
					InputBufferStride);

				// Gather Batch SubLayer Indices according to Top-2 Experts

				OperatorGatherTopTwoSubLayerBatchIndices(
					SparseMixtureOfExpertsInstance->SubLayerBatchIndices,
					SparseMixtureOfExpertsInstance->BatchSubLayerIndex0.GetData(),
					SparseMixtureOfExpertsInstance->BatchSubLayerIndex1.GetData(),
					SparseMixtureOfExpertsInstance->BatchSubLayerWeight0.GetData(),
					SparseMixtureOfExpertsInstance->BatchSubLayerWeight1.GetData(),
					SparseMixtureOfExpertsInstance->BatchSubLayerOutputIndex0.GetData(),
					SparseMixtureOfExpertsInstance->BatchSubLayerOutputIndex1.GetData(),
					SparseMixtureOfExpertsInstance->GatingOutputBuffer.GetData(),
					BatchSize,
					SubLayerNum,
					SubLayerNum);

				// Evaluate Each Sublayer on the associated batch items

				for (uint32 SubLayerIdx = 0; SubLayerIdx < SubLayerNum; SubLayerIdx++)
				{
					const uint32 SubLayerBatchSize = SparseMixtureOfExpertsInstance->SubLayerBatchIndices[SubLayerIdx].Num();

					if (SubLayerBatchSize == 0) { continue; }

					OperatorGather(
						SparseMixtureOfExpertsInstance->SubLayerInputBuffers[SubLayerIdx].GetData(),
						InputBuffer,
						SparseMixtureOfExpertsInstance->SubLayerBatchIndices[SubLayerIdx].GetData(),
						SubLayerBatchSize,
						InputSize,
						InputSize,
						InputBufferStride);

					SubLayers[SubLayerIdx]->Evaluate(
						SparseMixtureOfExpertsInstance->SubLayerInstances[SubLayerIdx].Get(),
						SparseMixtureOfExpertsInstance->SubLayerOutputBuffers[SubLayerIdx].GetData(),
						SparseMixtureOfExpertsInstance->SubLayerInputBuffers[SubLayerIdx].GetData(),
						SubLayerBatchSize,
						OutputSize,
						InputSize,
						OutputSize,
						InputSize);
				}

				// Do Weighted Sum of Top-2 Experts

				OperatorGatherTopTwoFromSubLayers(
					OutputBuffer,
					SparseMixtureOfExpertsInstance->BatchSubLayerIndex0.GetData(),
					SparseMixtureOfExpertsInstance->BatchSubLayerIndex1.GetData(),
					SparseMixtureOfExpertsInstance->BatchSubLayerWeight0.GetData(),
					SparseMixtureOfExpertsInstance->BatchSubLayerWeight1.GetData(),
					SparseMixtureOfExpertsInstance->BatchSubLayerOutputIndex0.GetData(),
					SparseMixtureOfExpertsInstance->BatchSubLayerOutputIndex1.GetData(),
					SparseMixtureOfExpertsInstance->SubLayerOutputBuffers,
					BatchSize,
					OutputBufferSize,
					OutputSize,
					OutputBufferStride);

				OperatorNanCheck(OutputBuffer, BatchSize, OutputBufferSize, OutputBufferStride);
			}

			uint32 InputSize = 0;
			uint32 OutputSize = 0;
			TSharedPtr<ILayer> GatingLayer;
			TArray<TSharedPtr<ILayer>, TInlineAllocator<32>> SubLayers;
		};

		FSparseMixtureOfExpertsLayerInstance::FSparseMixtureOfExpertsLayerInstance(const FSparseMixtureOfExpertsLayer& InSparseMixtureOfExpertsLayer)
			: SparseMixtureOfExpertsLayer(InSparseMixtureOfExpertsLayer)
		{
			GatingInstance = SparseMixtureOfExpertsLayer.GatingLayer->MakeInstance();

			const uint32 SubLayerNum = SparseMixtureOfExpertsLayer.SubLayers.Num();

			SubLayerBatchIndices.SetNum(SubLayerNum);
			SubLayerInputBuffers.SetNum(SubLayerNum);
			SubLayerOutputBuffers.SetNum(SubLayerNum);

			SubLayerInstances.Init(nullptr, SubLayerNum);
			for (uint32 LayerIdx = 0; LayerIdx < SubLayerNum; LayerIdx++)
			{
				SubLayerInstances[LayerIdx] = SparseMixtureOfExpertsLayer.SubLayers[LayerIdx]->MakeInstance();
			}
		}

		void FSparseMixtureOfExpertsLayerInstance::SetMaxBatchSize(const uint32 MaxBatchSize)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::FSparseMixtureOfExpertsLayerInstance::SetMaxBatchSize);

			const uint32 SubLayerNum = SparseMixtureOfExpertsLayer.SubLayers.Num();

			BatchSubLayerIndex0.SetNumUninitialized(MaxBatchSize, EAllowShrinking::No);
			BatchSubLayerIndex1.SetNumUninitialized(MaxBatchSize, EAllowShrinking::No);
			BatchSubLayerWeight0.SetNumUninitialized(MaxBatchSize, EAllowShrinking::No);
			BatchSubLayerWeight1.SetNumUninitialized(MaxBatchSize, EAllowShrinking::No);
			BatchSubLayerOutputIndex0.SetNumUninitialized(MaxBatchSize, EAllowShrinking::No);
			BatchSubLayerOutputIndex1.SetNumUninitialized(MaxBatchSize, EAllowShrinking::No);
			GatingOutputBuffer.SetNumUninitialized(MaxBatchSize * SubLayerNum, EAllowShrinking::No);

			// Propagate call to sub-layer instances

			if (GatingInstance)
			{
				GatingInstance->SetMaxBatchSize(MaxBatchSize);
			}
			
			for (uint32 SubLayerIdx = 0; SubLayerIdx < SubLayerNum; SubLayerIdx++)
			{
				if (SubLayerInstances[SubLayerIdx]) { SubLayerInstances[SubLayerIdx]->SetMaxBatchSize(MaxBatchSize); }

				SubLayerBatchIndices[SubLayerIdx].Empty(MaxBatchSize);
				SubLayerInputBuffers[SubLayerIdx].SetNumUninitialized(MaxBatchSize * SparseMixtureOfExpertsLayer.InputSize, EAllowShrinking::No);
				SubLayerOutputBuffers[SubLayerIdx].SetNumUninitialized(MaxBatchSize * SparseMixtureOfExpertsLayer.OutputSize, EAllowShrinking::No);
			}
		}

		//--------------------------------------------------------------------------

		struct FLayerNormLayer : public ILayer
		{
			virtual ELayerType GetLayerType() const override final { return ELayerType::LayerNorm; }
			virtual uint32 GetInputSize() const override final { return InputOutputSize; }
			virtual uint32 GetOutputSize() const override final { return InputOutputSize; }

			virtual void SerializationSize(uint64& InOutOffset) const override final
			{
				Serialization::Size(InOutOffset, InputOutputSize);
				Serialization::Size(InOutOffset, Offset);
				Serialization::Size(InOutOffset, Scale);
				Serialization::Size(InOutOffset, Epsilon);
			}

			virtual void SerializationLoad(uint64& InOutOffset, TConstArrayView<uint8> Data) override final
			{
				Serialization::Load(InOutOffset, InputOutputSize, Data);
				Serialization::Load(InOutOffset, Offset, Data, InputOutputSize);
				Serialization::Load(InOutOffset, Scale, Data, InputOutputSize);
				Serialization::Load(InOutOffset, Epsilon, Data);
			}

			virtual void SerializationSave(uint64& InOutOffset, TArrayView<uint8> Data) const override final
			{
				Serialization::Save(InOutOffset, InputOutputSize, Data);
				Serialization::Save(InOutOffset, Offset, Data);
				Serialization::Save(InOutOffset, Scale, Data);
				Serialization::Save(InOutOffset, Epsilon, Data);
			}

			virtual void Evaluate(
				ILayerInstance* Instance,
				float* OutputBuffer,
				const float* InputBuffer,
				const uint32 BatchSize,
				const uint32 OutputBufferSize,
				const uint32 InputBufferSize,
				const uint32 OutputBufferStride,
				const uint32 InputBufferStride) override final
			{
				NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::FLayerNormLayer::Evaluate);
				check(OutputBufferSize == GetOutputSize() && InputBufferSize == GetInputSize());
				check(OutputBufferStride >= GetOutputSize() && InputBufferStride >= GetInputSize());
				check(Instance == nullptr);
				OperatorNanCheck(InputBuffer, BatchSize, InputBufferSize, InputBufferStride);

				OperatorLayerNorm(
					OutputBuffer,
					InputBuffer,
					Offset.GetData(),
					Scale.GetData(),
					Epsilon,
					BatchSize,
					InputOutputSize,
					OutputBufferStride,
					InputBufferStride);

				OperatorNanCheck(OutputBuffer, BatchSize, OutputBufferSize, OutputBufferStride);
			}

			uint32 InputOutputSize = 0;
			TConstArrayView<float> Offset;
			TConstArrayView<float> Scale;
			float Epsilon = 1e-5f;
		};

		//--------------------------------------------------------------------------

		struct FLipschiztLinearLayer : public ILayer
		{
			virtual ELayerType GetLayerType() const override final { return ELayerType::LipschiztLinear; }
			virtual uint32 GetInputSize() const override final { return InputSize; }
			virtual uint32 GetOutputSize() const override final { return OutputSize; }

			virtual void SerializationSize(uint64& InOutOffset) const override final
			{
				Serialization::Size(InOutOffset, InputSize);
				Serialization::Size(InOutOffset, OutputSize);
				Serialization::Size(InOutOffset, Biases);
				Serialization::Size(InOutOffset, Weights);
			}

			virtual void SerializationLoad(uint64& InOutOffset, TConstArrayView<uint8> Data) override final
			{
				Serialization::Load(InOutOffset, InputSize, Data);
				Serialization::Load(InOutOffset, OutputSize, Data);
				Serialization::Load(InOutOffset, Biases, Data, OutputSize);
				Serialization::Load(InOutOffset, Weights, Data, InputSize * OutputSize);
			}

			virtual void SerializationSave(uint64& InOutOffset, TArrayView<uint8> Data) const override final
			{
				Serialization::Save(InOutOffset, InputSize, Data);
				Serialization::Save(InOutOffset, OutputSize, Data);
				Serialization::Save(InOutOffset, Biases, Data);
				Serialization::Save(InOutOffset, Weights, Data);
			}

			virtual void Evaluate(
				ILayerInstance* Instance,
				float* OutputBuffer,
				const float* InputBuffer,
				const uint32 BatchSize,
				const uint32 OutputBufferSize,
				const uint32 InputBufferSize,
				const uint32 OutputBufferStride,
				const uint32 InputBufferStride) override final
			{
				NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::FLipschiztLinearLayer::Evaluate);
				check(OutputBufferSize == GetOutputSize() && InputBufferSize == GetInputSize());
				check(OutputBufferStride >= GetOutputSize() && InputBufferStride >= GetInputSize());
				check(Instance == nullptr);
				OperatorNanCheck(InputBuffer, BatchSize, InputBufferSize, InputBufferStride);

				OperatorLinear(
					OutputBuffer,
					InputBuffer,
					Weights.GetData(),
					Biases.GetData(),
					BatchSize,
					OutputSize,
					InputSize,
					OutputBufferStride,
					InputBufferStride);

				OperatorNanCheck(OutputBuffer, BatchSize, OutputBufferSize, OutputBufferStride);
			}

			uint32 InputSize = 0;
			uint32 OutputSize = 0;
			TConstArrayView<float> Biases;
			TConstArrayView<float> Weights;
		};


		//--------------------------------------------------------------------------

		struct FTileLayer : public ILayer
		{
			virtual ELayerType GetLayerType() const override final { return ELayerType::Tile; }
			virtual uint32 GetInputSize() const override final { return InputSize; }
			virtual uint32 GetOutputSize() const override final { return Repeats * InputSize; }

			virtual void SerializationSize(uint64& InOutOffset) const override final
			{
				Serialization::Size(InOutOffset, InputSize);
				Serialization::Size(InOutOffset, Repeats);
			}

			virtual void SerializationLoad(uint64& InOutOffset, TConstArrayView<uint8> Data) override final
			{
				Serialization::Load(InOutOffset, InputSize, Data);
				Serialization::Load(InOutOffset, Repeats, Data);
			}

			virtual void SerializationSave(uint64& InOutOffset, TArrayView<uint8> Data) const override final
			{
				Serialization::Save(InOutOffset, InputSize, Data);
				Serialization::Save(InOutOffset, Repeats, Data);
			}

			virtual void Evaluate(
				ILayerInstance* Instance,
				float* OutputBuffer,
				const float* InputBuffer,
				const uint32 BatchSize,
				const uint32 OutputBufferSize,
				const uint32 InputBufferSize,
				const uint32 OutputBufferStride,
				const uint32 InputBufferStride) override final
			{
				NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::FTileLayer::Evaluate);
				check(OutputBufferSize == GetOutputSize() && InputBufferSize == GetInputSize());
				check(OutputBufferStride >= GetOutputSize() && InputBufferStride >= GetInputSize());
				check(Instance == nullptr);
				OperatorNanCheck(InputBuffer, BatchSize, InputBufferSize, InputBufferStride);

				OperatorTile(
					OutputBuffer,
					InputBuffer,
					BatchSize,
					InputSize,
					Repeats,
					OutputBufferStride,
					InputBufferStride);

				OperatorNanCheck(OutputBuffer, BatchSize, OutputBufferSize, OutputBufferStride);
			}

			uint32 Repeats = 1;
			uint32 InputSize = 0;
		};

		//--------------------------------------------------------------------------

		struct FSpreadLayer;

		struct FSpreadLayerInstance : public ILayerInstance
		{
			FSpreadLayerInstance(const FSpreadLayer& InSpreadLayer);

			virtual void SetMaxBatchSize(const uint32 MaxBatchSize) override final;
			virtual ELayerType GetLayerType() const override final { return ELayerType::Spread; }

			const FSpreadLayer& SpreadLayer;
			TArray<TSharedPtr<ILayerInstance>, TInlineAllocator<32>> Instances;
		};

		struct FSpreadLayer : public ILayer
		{
			virtual TSharedPtr<ILayerInstance> MakeInstance() const { return MakeShared<FSpreadLayerInstance>(*this); };
			virtual ELayerType GetLayerType() const override final { return ELayerType::Spread; }
			virtual uint32 GetInputSize() const override final { return InputSize; }
			virtual uint32 GetOutputSize() const override final { return TotalOutputSize; }

			virtual void SerializationSize(uint64& InOutOffset) const override final
			{
				Serialization::Size(InOutOffset, (uint32)Layers.Num());
				Serialization::Size(InOutOffset, InputSize);
				Serialization::Size(InOutOffset, OutputSizes);
				Serialization::Size(InOutOffset, Layers);
			}

			virtual void SerializationLoad(uint64& InOutOffset, TConstArrayView<uint8> Data) override final
			{
				uint32 LayerNum = 0;
				Serialization::Load(InOutOffset, LayerNum, Data);
				Serialization::Load(InOutOffset, InputSize, Data);
				Serialization::Load(InOutOffset, OutputSizes, Data, LayerNum);
				Layers.Init(nullptr, LayerNum);
				Serialization::Load(InOutOffset, Layers, Data);
				PostLoad();
			}

			void PostLoad()
			{
				const uint32 LayerNum = Layers.Num();
				OutputOffsets.SetNumUninitialized(LayerNum);

				TotalOutputSize = 0;
				for (uint32 LayerIdx = 0; LayerIdx < LayerNum; LayerIdx++)
				{
					OutputOffsets[LayerIdx] = TotalOutputSize;
					TotalOutputSize += OutputSizes[LayerIdx];
				}
			}

			virtual void SerializationSave(uint64& InOutOffset, TArrayView<uint8> Data) const override final
			{
				Serialization::Save(InOutOffset, (uint32)Layers.Num(), Data);
				Serialization::Save(InOutOffset, InputSize, Data);
				Serialization::Save(InOutOffset, OutputSizes, Data);
				Serialization::Save(InOutOffset, Layers, Data);
			}

			virtual void Evaluate(
				ILayerInstance* Instance,
				float* OutputBuffer,
				const float* InputBuffer,
				const uint32 BatchSize,
				const uint32 OutputBufferSize,
				const uint32 InputBufferSize,
				const uint32 OutputBufferStride,
				const uint32 InputBufferStride) override final
			{
				NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::FSpreadLayer::Evaluate);
				check(OutputBufferSize == GetOutputSize() && InputBufferSize == GetInputSize());
				check(OutputBufferStride >= GetOutputSize() && InputBufferStride >= GetInputSize());
				check(Instance && Instance->GetLayerType() == GetLayerType());

				FSpreadLayerInstance* SpreadInstance = StaticCast<FSpreadLayerInstance*>(Instance);

				OperatorNanCheck(InputBuffer, BatchSize, InputBufferSize, InputBufferStride);

				const int32 LayerNum = Layers.Num();

				for (int32 LayerIdx = 0; LayerIdx < LayerNum; LayerIdx++)
				{
					check(OutputOffsets[LayerIdx] + OutputSizes[LayerIdx] <= OutputBufferSize);

					Layers[LayerIdx]->Evaluate(
						SpreadInstance->Instances[LayerIdx].Get(),
						OutputBuffer + OutputOffsets[LayerIdx],
						InputBuffer,
						BatchSize,
						OutputSizes[LayerIdx],
						InputSize,
						OutputBufferStride,
						InputBufferStride);
				}

				OperatorNanCheck(OutputBuffer, BatchSize, OutputBufferSize, OutputBufferStride);
			}

			TConstArrayView<uint32> OutputSizes;
			TArray<TSharedPtr<ILayer>, TInlineAllocator<32>> Layers;

			uint32 InputSize = 0;
			uint32 TotalOutputSize = 0;
			TArray<uint32, TInlineAllocator<32>> OutputOffsets;
		};

		FSpreadLayerInstance::FSpreadLayerInstance(const FSpreadLayer& InSpreadLayer)
			: SpreadLayer(InSpreadLayer)
		{
			const uint32 LayerNum = SpreadLayer.Layers.Num();
			Instances.Init(nullptr, LayerNum);

			for (uint32 LayerIdx = 0; LayerIdx < LayerNum; LayerIdx++)
			{
				Instances[LayerIdx] = SpreadLayer.Layers[LayerIdx]->MakeInstance();
			}
		}

		void FSpreadLayerInstance::SetMaxBatchSize(const uint32 MaxBatchSize)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::FSpreadLayerInstance::SetMaxBatchSize);

			// Propagate call to sub-layer instances

			for (const TSharedPtr<ILayerInstance>& Instance : Instances)
			{
				if (Instance)
				{
					Instance->SetMaxBatchSize(MaxBatchSize);
				}
			}
		}


		//--------------------------------------------------------------------------

		struct FSliceLayer : public ILayer
		{
			virtual ELayerType GetLayerType() const override final { return ELayerType::Slice; }
			virtual uint32 GetInputSize() const override final { return InputSize; }
			virtual uint32 GetOutputSize() const override final { return SliceSize; }

			virtual void SerializationSize(uint64& InOutOffset) const override final
			{
				Serialization::Size(InOutOffset, InputSize);
				Serialization::Size(InOutOffset, SliceOffset);
				Serialization::Size(InOutOffset, SliceSize);
			}

			virtual void SerializationLoad(uint64& InOutOffset, TConstArrayView<uint8> Data) override final
			{
				Serialization::Load(InOutOffset, InputSize, Data);
				Serialization::Load(InOutOffset, SliceOffset, Data);
				Serialization::Load(InOutOffset, SliceSize, Data);
			}

			virtual void SerializationSave(uint64& InOutOffset, TArrayView<uint8> Data) const override final
			{
				Serialization::Save(InOutOffset, InputSize, Data);
				Serialization::Save(InOutOffset, SliceOffset, Data);
				Serialization::Save(InOutOffset, SliceSize, Data);
			}

			virtual void Evaluate(
				ILayerInstance* Instance,
				float* OutputBuffer,
				const float* InputBuffer,
				const uint32 BatchSize,
				const uint32 OutputBufferSize,
				const uint32 InputBufferSize,
				const uint32 OutputBufferStride,
				const uint32 InputBufferStride) override final
			{
				NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::FSliceLayer::Evaluate);
				check(OutputBufferSize == GetOutputSize() && InputBufferSize == GetInputSize());
				check(OutputBufferStride >= GetOutputSize() && InputBufferStride >= GetInputSize());
				check(Instance == nullptr);
				check(SliceOffset + SliceSize <= InputBufferSize);
				OperatorNanCheck(InputBuffer, BatchSize, InputBufferSize, InputBufferStride);

				OperatorCopy(
					OutputBuffer,
					InputBuffer + SliceOffset,
					BatchSize,
					SliceSize,
					OutputBufferStride,
					InputBufferStride);

				OperatorNanCheck(OutputBuffer, BatchSize, OutputBufferSize, OutputBufferStride);
			}

			uint32 InputSize = 0;
			uint32 SliceOffset = 0;
			uint32 SliceSize = 0;
		};


		//--------------------------------------------------------------------------

		struct FResidualLayer : public ILayer
		{
			virtual TSharedPtr<ILayerInstance> MakeInstance() const { return SubLayer->MakeInstance(); };
			virtual ELayerType GetLayerType() const override final { return ELayerType::Residual; }
			virtual uint32 GetInputSize() const override final { return InputOutputSize; }
			virtual uint32 GetOutputSize() const override final { return InputOutputSize; }

			virtual void SerializationSize(uint64& InOutOffset) const override final
			{
				Serialization::Size(InOutOffset, InputOutputSize);
				Serialization::Size(InOutOffset, SubLayer);
			}

			virtual void SerializationLoad(uint64& InOutOffset, TConstArrayView<uint8> Data) override final
			{
				Serialization::Load(InOutOffset, InputOutputSize, Data);
				Serialization::Load(InOutOffset, SubLayer, Data);
			}

			virtual void SerializationSave(uint64& InOutOffset, TArrayView<uint8> Data) const override final
			{
				Serialization::Save(InOutOffset, InputOutputSize, Data);
				Serialization::Save(InOutOffset, SubLayer, Data);
			}

			virtual void Evaluate(
				ILayerInstance* Instance,
				float* OutputBuffer,
				const float* InputBuffer,
				const uint32 BatchSize,
				const uint32 OutputBufferSize,
				const uint32 InputBufferSize,
				const uint32 OutputBufferStride,
				const uint32 InputBufferStride) override final
			{
				NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::FResidualLayer::Evaluate);
				check(OutputBufferSize == GetOutputSize() && InputBufferSize == GetInputSize());
				check(OutputBufferStride >= GetOutputSize() && InputBufferStride >= GetInputSize());

				OperatorNanCheck(InputBuffer, BatchSize, InputBufferSize, InputBufferStride);

				SubLayer->Evaluate(
					Instance,
					OutputBuffer,
					InputBuffer,
					BatchSize,
					OutputBufferSize,
					InputBufferSize,
					OutputBufferStride,
					InputBufferStride);

				OperatorAddInplace(
					OutputBuffer,
					InputBuffer,
					BatchSize,
					InputOutputSize,
					OutputBufferStride,
					InputBufferStride);

				OperatorNanCheck(OutputBuffer, BatchSize, OutputBufferSize, OutputBufferStride);
			}

			uint32 InputOutputSize = 0;
			TSharedPtr<ILayer> SubLayer;
		};


		//--------------------------------------------------------------------------

		struct FFiLMLayer;

		struct FFiLMInstance : public ILayerInstance
		{
			FFiLMInstance(const FFiLMLayer& InFiLMLayer);

			virtual void SetMaxBatchSize(const uint32 MaxBatchSize) override final;
			virtual ELayerType GetLayerType() const override final { return ELayerType::FiLM; }

			const FFiLMLayer& FiLMLayer;
			TSharedPtr<ILayerInstance> PrefixInstance;
			TSharedPtr<ILayerInstance> ConditionInstance;
			TSharedPtr<ILayerInstance> PostfixInstance;
			TArray<float> PrefixBuffer;
			TArray<float> ConditionBuffer;
		};

		struct FFiLMLayer : public ILayer
		{
			virtual TSharedPtr<ILayerInstance> MakeInstance() const { return MakeShared<FFiLMInstance>(*this); };
			virtual ELayerType GetLayerType() const override final { return ELayerType::FiLM; }
			virtual uint32 GetInputSize() const override final { return PrefixInputSize + ConditionInputSize; }
			virtual uint32 GetOutputSize() const override final { return PostfixOutputSize; }

			virtual void SerializationSize(uint64& InOutOffset) const override final
			{
				Serialization::Size(InOutOffset, PrefixInputSize);
				Serialization::Size(InOutOffset, PrefixOutputSize);
				Serialization::Size(InOutOffset, ConditionInputSize);
				Serialization::Size(InOutOffset, ConditionOutputSize);
				Serialization::Size(InOutOffset, PostfixInputSize);
				Serialization::Size(InOutOffset, PostfixOutputSize);
				Serialization::Size(InOutOffset, PrefixLayer);
				Serialization::Size(InOutOffset, ConditionLayer);
				Serialization::Size(InOutOffset, PostfixLayer);
			}

			virtual void SerializationLoad(uint64& InOutOffset, TConstArrayView<uint8> Data) override final
			{
				Serialization::Load(InOutOffset, PrefixInputSize, Data);
				Serialization::Load(InOutOffset, PrefixOutputSize, Data);
				Serialization::Load(InOutOffset, ConditionInputSize, Data);
				Serialization::Load(InOutOffset, ConditionOutputSize, Data);
				Serialization::Load(InOutOffset, PostfixInputSize, Data);
				Serialization::Load(InOutOffset, PostfixOutputSize, Data);
				Serialization::Load(InOutOffset, PrefixLayer, Data);
				Serialization::Load(InOutOffset, ConditionLayer, Data);
				Serialization::Load(InOutOffset, PostfixLayer, Data);
			}

			virtual void SerializationSave(uint64& InOutOffset, TArrayView<uint8> Data) const override final
			{
				Serialization::Save(InOutOffset, PrefixInputSize, Data);
				Serialization::Save(InOutOffset, PrefixOutputSize, Data);
				Serialization::Save(InOutOffset, ConditionInputSize, Data);
				Serialization::Save(InOutOffset, ConditionOutputSize, Data);
				Serialization::Save(InOutOffset, PostfixInputSize, Data);
				Serialization::Save(InOutOffset, PostfixOutputSize, Data);
				Serialization::Save(InOutOffset, PrefixLayer, Data);
				Serialization::Save(InOutOffset, ConditionLayer, Data);
				Serialization::Save(InOutOffset, PostfixLayer, Data);
			}

			virtual void Evaluate(
				ILayerInstance* Instance,
				float* OutputBuffer,
				const float* InputBuffer,
				const uint32 BatchSize,
				const uint32 OutputBufferSize,
				const uint32 InputBufferSize,
				const uint32 OutputBufferStride,
				const uint32 InputBufferStride) override final
			{
				NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::FFiLMLayer::Evaluate);
				check(OutputBufferSize == GetOutputSize() && InputBufferSize == GetInputSize());
				check(OutputBufferStride >= GetOutputSize() && InputBufferStride >= GetInputSize());
				check(PrefixOutputSize * 2 == ConditionOutputSize);
				check(PostfixInputSize == PrefixOutputSize);
				check(Instance && Instance->GetLayerType() == GetLayerType());

				FFiLMInstance* FiLMInstance = StaticCast<FFiLMInstance*>(Instance);

				OperatorNanCheck(InputBuffer, BatchSize, InputBufferSize, InputBufferStride);

				PrefixLayer->Evaluate(
					FiLMInstance->PrefixInstance.Get(),
					FiLMInstance->PrefixBuffer.GetData(),
					InputBuffer,
					BatchSize,
					PrefixOutputSize,
					PrefixInputSize,
					PrefixOutputSize,
					InputBufferStride);

				ConditionLayer->Evaluate(
					FiLMInstance->ConditionInstance.Get(),
					FiLMInstance->ConditionBuffer.GetData(),
					InputBuffer + PrefixInputSize,
					BatchSize,
					ConditionOutputSize,
					ConditionInputSize,
					ConditionOutputSize,
					InputBufferStride);

				OperatorLayerFiLM(
					FiLMInstance->PrefixBuffer.GetData(),
					FiLMInstance->ConditionBuffer.GetData(),
					BatchSize,
					PrefixOutputSize,
					PrefixOutputSize,
					ConditionOutputSize);

				PostfixLayer->Evaluate(
					FiLMInstance->PostfixInstance.Get(),
					OutputBuffer,
					FiLMInstance->PrefixBuffer.GetData(),
					BatchSize,
					OutputBufferSize,
					PostfixInputSize,
					OutputBufferStride,
					PostfixInputSize);

				OperatorNanCheck(OutputBuffer, BatchSize, OutputBufferSize, OutputBufferStride);
			}

			uint32 PrefixInputSize = 0;
			uint32 PrefixOutputSize = 0;
			uint32 ConditionInputSize = 0;
			uint32 ConditionOutputSize = 0;
			uint32 PostfixInputSize = 0;
			uint32 PostfixOutputSize = 0;

			TSharedPtr<ILayer> PrefixLayer;
			TSharedPtr<ILayer> ConditionLayer;
			TSharedPtr<ILayer> PostfixLayer;
		};

		FFiLMInstance::FFiLMInstance(const FFiLMLayer& InFiLMLayer)
			: FiLMLayer(InFiLMLayer)
		{
			PrefixInstance = InFiLMLayer.PrefixLayer->MakeInstance();
			ConditionInstance = InFiLMLayer.ConditionLayer->MakeInstance();
			PostfixInstance = InFiLMLayer.PostfixLayer->MakeInstance();
		}

		void FFiLMInstance::SetMaxBatchSize(const uint32 MaxBatchSize)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::FFiLMInstance::SetMaxBatchSize);

			if (PrefixInstance) { PrefixInstance->SetMaxBatchSize(MaxBatchSize); }
			if (ConditionInstance) { ConditionInstance->SetMaxBatchSize(MaxBatchSize); }
			if (PostfixInstance) { PostfixInstance->SetMaxBatchSize(MaxBatchSize); }

			PrefixBuffer.SetNumUninitialized(MaxBatchSize * FiLMLayer.PrefixOutputSize, EAllowShrinking::No);
			ConditionBuffer.SetNumUninitialized(MaxBatchSize * FiLMLayer.ConditionOutputSize, EAllowShrinking::No);
		}


		//--------------------------------------------------------------------------

		struct FGRUCellLayer;

		struct FGRUCellInstance : public ILayerInstance
		{
			FGRUCellInstance(const FGRUCellLayer& InGRUCellLayer);

			virtual void SetMaxBatchSize(const uint32 MaxBatchSize) override final;
			virtual ELayerType GetLayerType() const override final { return ELayerType::GRUCell; }

			const FGRUCellLayer& GRUCellLayer;
			TSharedPtr<ILayerInstance> RememberInstance;
			TSharedPtr<ILayerInstance> UpdateInstance;
			TSharedPtr<ILayerInstance> ActivationInstance;
			TArray<float> RememberGateBuffer;
			TArray<float> UpdateGateBuffer;
			TArray<float> ActivationGateBuffer;
			TArray<float> InputMaskBuffer;
		};

		struct FGRUCellLayer : public ILayer
		{
			virtual TSharedPtr<ILayerInstance> MakeInstance() const { return MakeShared<FGRUCellInstance>(*this); };
			virtual ELayerType GetLayerType() const override final { return ELayerType::GRUCell; }
			virtual uint32 GetInputSize() const override final { return InputSize + OutputSize; }
			virtual uint32 GetOutputSize() const override final { return OutputSize + OutputSize; }

			virtual void SerializationSize(uint64& InOutOffset) const override final
			{
				Serialization::Size(InOutOffset, InputSize);
				Serialization::Size(InOutOffset, OutputSize);
				Serialization::Size(InOutOffset, RememberLayer);
				Serialization::Size(InOutOffset, UpdateLayer);
				Serialization::Size(InOutOffset, ActivationLayer);
			}

			virtual void SerializationLoad(uint64& InOutOffset, TConstArrayView<uint8> Data) override final
			{
				Serialization::Load(InOutOffset, InputSize, Data);
				Serialization::Load(InOutOffset, OutputSize, Data);
				Serialization::Load(InOutOffset, RememberLayer, Data);
				Serialization::Load(InOutOffset, UpdateLayer, Data);
				Serialization::Load(InOutOffset, ActivationLayer, Data);
			}

			virtual void SerializationSave(uint64& InOutOffset, TArrayView<uint8> Data) const override final
			{
				Serialization::Save(InOutOffset, InputSize, Data);
				Serialization::Save(InOutOffset, OutputSize, Data);
				Serialization::Save(InOutOffset, RememberLayer, Data);
				Serialization::Save(InOutOffset, UpdateLayer, Data);
				Serialization::Save(InOutOffset, ActivationLayer, Data);
			}

			virtual void Evaluate(
				ILayerInstance* Instance,
				float* OutputBuffer,
				const float* InputBuffer,
				const uint32 BatchSize,
				const uint32 OutputBufferSize,
				const uint32 InputBufferSize,
				const uint32 OutputBufferStride,
				const uint32 InputBufferStride) override final
			{
				NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::FGRUCellLayer::Evaluate);
				check(OutputBufferSize == GetOutputSize() && InputBufferSize == GetInputSize());
				check(OutputBufferStride >= GetOutputSize() && InputBufferStride >= GetInputSize());
				check(Instance && Instance->GetLayerType() == GetLayerType());

				FGRUCellInstance* GRUCellInstance = StaticCast<FGRUCellInstance*>(Instance);

				OperatorNanCheck(InputBuffer, BatchSize, InputBufferSize, InputBufferStride);

				// Remember Gate

				RememberLayer->Evaluate(
					GRUCellInstance->RememberInstance.Get(),
					GRUCellInstance->RememberGateBuffer.GetData(),
					InputBuffer,
					BatchSize,
					OutputSize,
					InputBufferSize,
					OutputSize,
					InputBufferStride);

				// Update Gate

				UpdateLayer->Evaluate(
					GRUCellInstance->UpdateInstance.Get(),
					GRUCellInstance->UpdateGateBuffer.GetData(),
					InputBuffer,
					BatchSize,
					OutputSize,
					InputBufferSize,
					OutputSize,
					InputBufferStride);

				// Mask Input

				OperatorGRUMaskInput(
					GRUCellInstance->InputMaskBuffer.GetData(),
					InputBuffer,
					GRUCellInstance->RememberGateBuffer.GetData(),
					BatchSize,
					OutputSize,
					InputSize,
					InputSize + OutputSize,
					InputBufferStride,
					OutputSize);

				// Activation Gate

				ActivationLayer->Evaluate(
					GRUCellInstance->ActivationInstance.Get(),
					GRUCellInstance->ActivationGateBuffer.GetData(),
					GRUCellInstance->InputMaskBuffer.GetData(),
					BatchSize,
					OutputSize,
					InputSize + OutputSize,
					OutputSize,
					InputSize + OutputSize);

				// Compute Output

				OperatorGRUCellUpdateOutput(
					OutputBuffer,
					InputBuffer,
					GRUCellInstance->UpdateGateBuffer.GetData(),
					GRUCellInstance->ActivationGateBuffer.GetData(),
					BatchSize,
					OutputSize,
					InputSize,
					OutputBufferStride,
					InputBufferStride,
					OutputSize,
					OutputSize);

				OperatorNanCheck(OutputBuffer, BatchSize, OutputBufferSize, OutputBufferStride);
			}

			uint32 InputSize = 0;
			uint32 OutputSize = 0;
			TSharedPtr<ILayer> RememberLayer;
			TSharedPtr<ILayer> UpdateLayer;
			TSharedPtr<ILayer> ActivationLayer;
		};

		FGRUCellInstance::FGRUCellInstance(const FGRUCellLayer& InGRUCellLayer)
			: GRUCellLayer(InGRUCellLayer)
		{
			RememberInstance = GRUCellLayer.RememberLayer->MakeInstance();
			UpdateInstance = GRUCellLayer.UpdateLayer->MakeInstance();
			ActivationInstance = GRUCellLayer.ActivationLayer->MakeInstance();
		}

		void FGRUCellInstance::SetMaxBatchSize(const uint32 MaxBatchSize)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::FGRUCellInstance::SetMaxBatchSize);

			if (RememberInstance) { RememberInstance->SetMaxBatchSize(MaxBatchSize); }
			if (UpdateInstance) { UpdateInstance->SetMaxBatchSize(MaxBatchSize); }
			if (ActivationInstance) { ActivationInstance->SetMaxBatchSize(MaxBatchSize); }

			RememberGateBuffer.SetNumUninitialized(MaxBatchSize * GRUCellLayer.OutputSize, EAllowShrinking::No);
			UpdateGateBuffer.SetNumUninitialized(MaxBatchSize * GRUCellLayer.OutputSize, EAllowShrinking::No);
			ActivationGateBuffer.SetNumUninitialized(MaxBatchSize * GRUCellLayer.OutputSize, EAllowShrinking::No);
			InputMaskBuffer.SetNumUninitialized(MaxBatchSize * (GRUCellLayer.InputSize + GRUCellLayer.OutputSize), EAllowShrinking::No);
		}


		//--------------------------------------------------------------------------

		struct FConv1dLayer;

		struct FConv1dInstance : public ILayerInstance
		{
			FConv1dInstance(const FConv1dLayer& InConv1dLayer);

			void SetMaxBatchSize(const uint32 MaxBatchSize) override final;
			virtual ELayerType GetLayerType() const override final { return ELayerType::Conv1d; }

			const FConv1dLayer& Conv1dLayer;
			TSharedPtr<ILayerInstance> SubLayerInstance;
			TArray<float> SubLayerOutputBuffer;
		};

		struct FConv1dLayer : public ILayer
		{
			virtual TSharedPtr<ILayerInstance> MakeInstance() const { return MakeShared<FConv1dInstance>(*this); };
			virtual ELayerType GetLayerType() const override final { return ELayerType::Conv1d; }
			virtual uint32 GetInputSize() const override final { return SubLayer->GetInputSize(); }
			virtual uint32 GetOutputSize() const override final 
			{ 
				uint32 OutputLength = InputLength + 2 * Padding - KernelSize + 1;
				return OutputLength * OutChannels;
			}
			uint32 GetPaddingMode() const { return static_cast<uint32_t>(PaddingMode); }
			
			void SerializationLoadPaddingMode(uint64& InOutOffset, TConstArrayView<uint8> Data) 
			{
				uint32 LoadedMode;
				Serialization::Load(InOutOffset, LoadedMode, Data);
				PaddingMode = static_cast<FModelBuilder::EPaddingMode>(LoadedMode);
			}

			virtual void SerializationSize(uint64& InOutOffset) const override final
			{
				Serialization::Size(InOutOffset, InputLength);
				Serialization::Size(InOutOffset, InChannels);
				Serialization::Size(InOutOffset, OutChannels);
				Serialization::Size(InOutOffset, KernelSize);
				Serialization::Size(InOutOffset, Padding);
				Serialization::Size(InOutOffset, GetPaddingMode());
				Serialization::Size(InOutOffset, Weights);
				Serialization::Size(InOutOffset, Biases);
				Serialization::Size(InOutOffset, SubLayer);
			}

			virtual void SerializationLoad(uint64& InOutOffset, TConstArrayView<uint8> Data) override final
			{
				Serialization::Load(InOutOffset, InputLength, Data);
				Serialization::Load(InOutOffset, InChannels, Data);
				Serialization::Load(InOutOffset, OutChannels, Data);
				Serialization::Load(InOutOffset, KernelSize, Data);
				Serialization::Load(InOutOffset, Padding, Data);
				SerializationLoadPaddingMode(InOutOffset, Data);
				Serialization::Load(InOutOffset, Weights, Data, KernelSize * InChannels * OutChannels);
				Serialization::Load(InOutOffset, Biases, Data, OutChannels);
				Serialization::Load(InOutOffset, SubLayer, Data);
			}

			virtual void SerializationSave(uint64& InOutOffset, TArrayView<uint8> Data) const override final
			{
				Serialization::Save(InOutOffset, InputLength, Data);
				Serialization::Save(InOutOffset, InChannels, Data);
				Serialization::Save(InOutOffset, OutChannels, Data);
				Serialization::Save(InOutOffset, KernelSize, Data);
				Serialization::Save(InOutOffset, Padding, Data);
				Serialization::Save(InOutOffset, GetPaddingMode(), Data);
				Serialization::Save(InOutOffset, Weights, Data);
				Serialization::Save(InOutOffset, Biases, Data);
				Serialization::Save(InOutOffset, SubLayer, Data);
			}

			virtual void Evaluate(
				ILayerInstance* Instance,
				float* OutputBuffer,
				const float* InputBuffer,
				const uint32 BatchSize,
				const uint32 OutputBufferSize,
				const uint32 InputBufferSize,
				const uint32 OutputBufferStride,
				const uint32 InputBufferStride) override final
			{
				NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::FConv1dLayer::Evaluate);
				check(OutputBufferSize == GetOutputSize() && InputBufferSize == GetInputSize());
				check(OutputBufferStride >= GetOutputSize() && InputBufferStride >= GetInputSize());
				check(Instance && Instance->GetLayerType() == GetLayerType());

				FConv1dInstance* Conv1dLayerInstance = StaticCast<FConv1dInstance*>(Instance);

				OperatorNanCheck(InputBuffer, BatchSize, InputBufferSize, InputBufferStride);

				SubLayer->Evaluate(
					Conv1dLayerInstance->SubLayerInstance.Get(),
					Conv1dLayerInstance->SubLayerOutputBuffer.GetData(),
					InputBuffer,
					BatchSize,
					SubLayer->GetOutputSize(),
					SubLayer->GetInputSize(),
					SubLayer->GetOutputSize(), 
					SubLayer->GetInputSize());

				OperatorConv1d(
					OutputBuffer,
					Conv1dLayerInstance->SubLayerOutputBuffer.GetData(),
					Weights.GetData(),
					Biases.GetData(),
					BatchSize,
					InChannels,
					OutChannels,
					InputLength,
					KernelSize,
					Padding,
					PaddingMode,
					OutputBufferStride,
					InputLength * InChannels);

				OperatorNanCheck(OutputBuffer, BatchSize, OutputBufferSize, OutputBufferStride);
			}
			uint32 InputLength = 0, InChannels = 0, OutChannels = 0, KernelSize = 0, Padding = 0;
			FModelBuilder::EPaddingMode PaddingMode;
			TConstArrayView<float> Weights, Biases;
			TSharedPtr<ILayer> SubLayer;
		};

		FConv1dInstance::FConv1dInstance(const FConv1dLayer& InConv1dLayer) : Conv1dLayer(InConv1dLayer)
		{
			SubLayerInstance = Conv1dLayer.SubLayer->MakeInstance();
		}

		void FConv1dInstance::SetMaxBatchSize(const uint32 MaxBatchSize)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::FConv1dLayerInstance::SetMaxBatchSize);
			SubLayerInstance->SetMaxBatchSize(MaxBatchSize);
			SubLayerOutputBuffer.SetNumUninitialized(MaxBatchSize * Conv1dLayer.SubLayer->GetOutputSize(), EAllowShrinking::No);
		}
		//--------------------------------------------------------------------------

		struct FConv2dLayer;

		struct FConv2dInstance : public ILayerInstance
		{
			FConv2dInstance(const FConv2dLayer& InConv2dLayer);

			void SetMaxBatchSize(const uint32 MaxBatchSize) override final;
			virtual ELayerType GetLayerType() const override final { return ELayerType::Conv2d; }

			const FConv2dLayer& Conv2dLayer;
			TSharedPtr<ILayerInstance> SubLayerInstance;
			TArray<float> SubLayerOutputBuffer;
		};

		struct FConv2dLayer : public ILayer
		{
			virtual TSharedPtr<ILayerInstance> MakeInstance() const { return MakeShared<FConv2dInstance>(*this); };
			virtual ELayerType GetLayerType() const override final { return ELayerType::Conv2d; }
			virtual uint32 GetInputSize() const override final { return SubLayer->GetInputSize(); }
			virtual uint32 GetOutputSize() const override final
			{
				uint32 OutputHeight = (InputHeight + 2 * Padding - KernelSize) / Stride + 1;
				uint32 OutputWidth = (InputWidth + 2 * Padding - KernelSize) / Stride + 1;
				return OutputHeight * OutputWidth * OutChannels;
			}
			uint32 GetPaddingMode() const { return static_cast<uint32_t>(PaddingMode); }

			void SerializationLoadPaddingMode(uint64& InOutOffset, TConstArrayView<uint8> Data)
			{
				uint32 LoadedMode;
				Serialization::Load(InOutOffset, LoadedMode, Data);
				PaddingMode = static_cast<FModelBuilder::EPaddingMode>(LoadedMode);
			}

			virtual void SerializationSize(uint64& InOutOffset) const override final
			{
				Serialization::Size(InOutOffset, InputHeight);
				Serialization::Size(InOutOffset, InputWidth);
				Serialization::Size(InOutOffset, InChannels);
				Serialization::Size(InOutOffset, OutChannels);
				Serialization::Size(InOutOffset, KernelSize);
				Serialization::Size(InOutOffset, Stride);
				Serialization::Size(InOutOffset, Padding);
				Serialization::Size(InOutOffset, GetPaddingMode());
				Serialization::Size(InOutOffset, Weights);
				Serialization::Size(InOutOffset, Biases);
				Serialization::Size(InOutOffset, SubLayer);
			}

			virtual void SerializationLoad(uint64& InOutOffset, TConstArrayView<uint8> Data) override final
			{
				Serialization::Load(InOutOffset, InputHeight, Data);
				Serialization::Load(InOutOffset, InputWidth, Data);
				Serialization::Load(InOutOffset, InChannels, Data);
				Serialization::Load(InOutOffset, OutChannels, Data);
				Serialization::Load(InOutOffset, KernelSize, Data);
				Serialization::Load(InOutOffset, Stride, Data);
				Serialization::Load(InOutOffset, Padding, Data);
				SerializationLoadPaddingMode(InOutOffset, Data);
				Serialization::Load(InOutOffset, Weights, Data, KernelSize * KernelSize * InChannels * OutChannels);
				Serialization::Load(InOutOffset, Biases, Data, OutChannels);
				Serialization::Load(InOutOffset, SubLayer, Data);
			}

			virtual void SerializationSave(uint64& InOutOffset, TArrayView<uint8> Data) const override final
			{
				Serialization::Save(InOutOffset, InputHeight, Data);
				Serialization::Save(InOutOffset, InputWidth, Data);
				Serialization::Save(InOutOffset, InChannels, Data);
				Serialization::Save(InOutOffset, OutChannels, Data);
				Serialization::Save(InOutOffset, KernelSize, Data);
				Serialization::Save(InOutOffset, Stride, Data);
				Serialization::Save(InOutOffset, Padding, Data);
				Serialization::Save(InOutOffset, GetPaddingMode(), Data);
				Serialization::Save(InOutOffset, Weights, Data);
				Serialization::Save(InOutOffset, Biases, Data);
				Serialization::Save(InOutOffset, SubLayer, Data);
			}

			virtual void Evaluate(
				ILayerInstance* Instance,
				float* OutputBuffer,
				const float* InputBuffer,
				const uint32 BatchSize,
				const uint32 OutputBufferSize,
				const uint32 InputBufferSize,
				const uint32 OutputBufferStride,
				const uint32 InputBufferStride) override final
			{
				NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::FConv2dLayer::Evaluate);
				check(OutputBufferSize == GetOutputSize() && InputBufferSize == GetInputSize());
				check(OutputBufferStride >= GetOutputSize() && InputBufferStride >= GetInputSize());
				check(Instance && Instance->GetLayerType() == GetLayerType());

				FConv2dInstance* Conv2dLayerInstance = StaticCast<FConv2dInstance*>(Instance);

				OperatorNanCheck(InputBuffer, BatchSize, InputBufferSize, InputBufferStride);

				SubLayer->Evaluate(
					Conv2dLayerInstance->SubLayerInstance.Get(),
					Conv2dLayerInstance->SubLayerOutputBuffer.GetData(),
					InputBuffer,
					BatchSize,
					SubLayer->GetOutputSize(),
					SubLayer->GetInputSize(),
					SubLayer->GetOutputSize(),
					SubLayer->GetInputSize());

				OperatorConv2d(
					OutputBuffer,
					Conv2dLayerInstance->SubLayerOutputBuffer.GetData(),
					Weights.GetData(),
					Biases.GetData(),
					BatchSize,
					InChannels,
					OutChannels,
					InputHeight,
					InputWidth,
					KernelSize,
					Stride,
					Padding,
					PaddingMode,
					OutputBufferStride,
					InputHeight * InputWidth * InChannels);

				OperatorNanCheck(OutputBuffer, BatchSize, OutputBufferSize, OutputBufferStride);
			}
			uint32 InputHeight = 0, InputWidth = 0, InChannels = 0, OutChannels = 0, KernelSize = 0, Stride = 0, Padding = 0;
			FModelBuilder::EPaddingMode PaddingMode;
			TConstArrayView<float> Weights, Biases;
			TSharedPtr<ILayer> SubLayer;
		};

		FConv2dInstance::FConv2dInstance(const FConv2dLayer& InConv2dLayer) : Conv2dLayer(InConv2dLayer)
		{
			SubLayerInstance = Conv2dLayer.SubLayer->MakeInstance();
		}

		void FConv2dInstance::SetMaxBatchSize(const uint32 MaxBatchSize)
		{
			NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::FConv2dLayerInstance::SetMaxBatchSize);
			SubLayerInstance->SetMaxBatchSize(MaxBatchSize);
			SubLayerOutputBuffer.SetNumUninitialized(MaxBatchSize * Conv2dLayer.SubLayer->GetOutputSize(), EAllowShrinking::No);
		}

		//--------------------------------------------------------------------------

		struct FSoftmaxLayer : public ILayer
		{
			virtual ELayerType GetLayerType() const override final { return ELayerType::Softmax; }
			virtual uint32 GetInputSize() const override final { return InputOutputSize; }
			virtual uint32 GetOutputSize() const override final { return InputOutputSize; }

			virtual void SerializationSize(uint64& InOutOffset) const override final
			{
				Serialization::Size(InOutOffset, InputOutputSize);
			}

			virtual void SerializationLoad(uint64& InOutOffset, TConstArrayView<uint8> Data) override final
			{
				Serialization::Load(InOutOffset, InputOutputSize, Data);
			}

			virtual void SerializationSave(uint64& InOutOffset, TArrayView<uint8> Data) const override final
			{
				Serialization::Save(InOutOffset, InputOutputSize, Data);
			}

			virtual void Evaluate(
				ILayerInstance* Instance,
				float* OutputBuffer,
				const float* InputBuffer,
				const uint32 BatchSize,
				const uint32 OutputBufferSize,
				const uint32 InputBufferSize,
				const uint32 OutputBufferStride,
				const uint32 InputBufferStride) override final
			{
				NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::Private::FSoftmaxLayer::Evaluate);
				check(OutputBufferSize == GetOutputSize() && InputBufferSize == GetInputSize());
				check(OutputBufferStride >= GetOutputSize() && InputBufferStride >= GetInputSize());
				check(Instance == nullptr);
				OperatorNanCheck(InputBuffer, BatchSize, InputBufferSize, InputBufferStride);

				OperatorSoftmax(
					OutputBuffer,
					InputBuffer,
					BatchSize,
					InputOutputSize,
					OutputBufferStride,
					InputBufferStride);

				OperatorNanCheck(OutputBuffer, BatchSize, OutputBufferSize, OutputBufferStride);
			}

			uint32 InputOutputSize = 0;
		};


		//--------------------------------------------------------------------------
		// Layer Serialization
		//--------------------------------------------------------------------------

		namespace Serialization
		{
			static inline void Size(uint64& InOutOffset, const TSharedPtr<ILayer>& InLayer)
			{
				Serialization::Size(InOutOffset, (uint32)InLayer->GetLayerType());
				InLayer->SerializationSize(InOutOffset);
			}

			static inline void Size(uint64& InOutOffset, const TConstArrayView<TSharedPtr<ILayer>> InLayers)
			{
				for (const TSharedPtr<ILayer>& Layer : InLayers) { Serialization::Size(InOutOffset, Layer); }
			}

			static inline void Load(uint64& InOutOffset, TSharedPtr<ILayer>& OutLayer, TConstArrayView<uint8> Data)
			{
				uint32 LayerTypeId = (uint32)ELayerType::Invalid;
				Serialization::Load(InOutOffset, LayerTypeId, Data);

				checkf((ELayerType)LayerTypeId != ELayerType::Invalid, TEXT("Invalid Layer"));

				if (OutLayer == nullptr || OutLayer->GetLayerType() != (ELayerType)LayerTypeId)
				{
					switch ((ELayerType)LayerTypeId)
					{
					case ELayerType::Sequence: OutLayer = MakeShared<FSequenceLayer>(); break;
					case ELayerType::Normalize: OutLayer = MakeShared<FNormalizeLayer>(); break;
					case ELayerType::Denormalize: OutLayer = MakeShared<FDenormalizeLayer>(); break;
					case ELayerType::Linear: OutLayer = MakeShared<FLinearLayer>(); break;
					case ELayerType::CompressedLinear: OutLayer = MakeShared<FCompressedLinearLayer>(); break;
					case ELayerType::MultiLinear: OutLayer = MakeShared<FMultiLinearLayer>(); break;
					case ELayerType::ReLU: OutLayer = MakeShared<FReLULayer>(); break;
					case ELayerType::ELU: OutLayer = MakeShared<FELULayer>(); break;
					case ELayerType::TanH: OutLayer = MakeShared<FTanHLayer>(); break;
					case ELayerType::PReLU: OutLayer = MakeShared<FPReLULayer>(); break;
					case ELayerType::MemoryCell: OutLayer = MakeShared<FMemoryCellLayer>(); break;
					case ELayerType::Copy: OutLayer = MakeShared<FCopyLayer>(); break;
					case ELayerType::Concat: OutLayer = MakeShared<FConcatLayer>(); break;
					case ELayerType::Array: OutLayer = MakeShared<FArrayLayer>(); break;
					case ELayerType::AggregateSet: OutLayer = MakeShared<FAggregateSetLayer>(); break;
					case ELayerType::AggregateOrExclusive: OutLayer = MakeShared<FAggregateOrExclusiveLayer>(); break;
					case ELayerType::AggregateOrInclusive: OutLayer = MakeShared<FAggregateOrInclusiveLayer>(); break;
					case ELayerType::Clamp: OutLayer = MakeShared<FClampLayer>(); break;
					case ELayerType::SparseMixtureOfExperts: OutLayer = MakeShared<FSparseMixtureOfExpertsLayer>(); break;
					case ELayerType::GELU: OutLayer = MakeShared<FGELULayer>(); break;
					case ELayerType::LayerNorm: OutLayer = MakeShared<FLayerNormLayer>(); break;
					case ELayerType::LipschiztLinear: OutLayer = MakeShared<FLipschiztLinearLayer>(); break;
					case ELayerType::Tile: OutLayer = MakeShared<FTileLayer>(); break;
					case ELayerType::Spread: OutLayer = MakeShared<FSpreadLayer>(); break;
					case ELayerType::Slice: OutLayer = MakeShared<FSliceLayer>(); break;
					case ELayerType::Residual: OutLayer = MakeShared<FResidualLayer>(); break;
					case ELayerType::FiLM: OutLayer = MakeShared<FFiLMLayer>(); break;
					case ELayerType::GRUCell: OutLayer = MakeShared<FGRUCellLayer>(); break;
					case ELayerType::Conv1d: OutLayer = MakeShared<FConv1dLayer>(); break;
					case ELayerType::Conv2d: OutLayer = MakeShared<FConv2dLayer>(); break;
					case ELayerType::Softmax: OutLayer = MakeShared<FSoftmaxLayer>(); break;
					default: checkf(false, TEXT("Unknown Layer Id %i"), LayerTypeId);
					}
				}

				OutLayer->SerializationLoad(InOutOffset, Data);
			}

			static inline void Load(uint64& InOutOffset, TArrayView<TSharedPtr<ILayer>> OutLayers, TConstArrayView<uint8> Data)
			{
				for (TSharedPtr<ILayer>& Layer : OutLayers) { Serialization::Load(InOutOffset, Layer, Data); }
			}

			static inline void Save(uint64& InOutOffset, const TSharedPtr<ILayer>& InLayer, TArrayView<uint8> Data)
			{
				Serialization::Save(InOutOffset, (uint32)InLayer->GetLayerType(), Data);
				InLayer->SerializationSave(InOutOffset, Data);
			}

			static inline void Save(uint64& InOutOffset, const TConstArrayView<TSharedPtr<ILayer>> InLayers, TArrayView<uint8> Data)
			{
				for (const TSharedPtr<ILayer>& Layer : InLayers) { Serialization::Save(InOutOffset, Layer, Data); }
			}
		}
	}

	//--------------------------------------------------------------------------
	// NNE Interface Implementation
	//--------------------------------------------------------------------------

	FModelInstanceCPU::FModelInstanceCPU(const TSharedPtr<FModelCPU>& InModel)
		: Model(InModel)
		, InputTensorDesc(FTensorDesc::Make(TEXT("Input"), FSymbolicTensorShape::Make({ -1, (int32)Model->Layer->GetInputSize() }), ENNETensorDataType::Float))
		, OutputTensorDesc(FTensorDesc::Make(TEXT("Output"), FSymbolicTensorShape::Make({ -1, (int32)Model->Layer->GetOutputSize() }), ENNETensorDataType::Float))
		, InputTensorShape(FTensorShape::Make({ 0, Model->Layer->GetInputSize() }))
		, OutputTensorShape(FTensorShape::Make({ 0, Model->Layer->GetOutputSize() }))
		, Instance(Model->Layer->MakeInstance())
		, BatchSize(0)
		, InputSize(Model->Layer->GetInputSize())
		, OutputSize(Model->Layer->GetOutputSize())
	{}

	FModelInstanceCPU::ESetInputTensorShapesStatus FModelInstanceCPU::SetInputTensorShapes(TConstArrayView<FTensorShape> InInputShapes)
	{
		NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::FModelInstanceCPU::SetInputTensorShapes);

		if (!ensureMsgf(InInputShapes.Num() == 1, TEXT("Basic CPU Inference only supports single input tensor.")))
		{
			return ESetInputTensorShapesStatus::Fail;
		}

		const FTensorShape& InputShape = InInputShapes[0];

		if (!ensureMsgf(InputShape.Rank() == 2, TEXT("Basic CPU Inference only supports rank 2 input tensors.")))
		{
			return ESetInputTensorShapesStatus::Fail;
		}

		const uint32 InputInputSize = InputShape.GetData()[1];

		if (!ensureMsgf(InputInputSize == InputSize, TEXT("Input tensor shape does not match model input size. Got %i, expected %i."), InputInputSize, InputSize))
		{
			return ESetInputTensorShapesStatus::Fail;
		}

		const uint32 InputBatchSize = InputShape.GetData()[0];

		if (InputBatchSize != BatchSize)
		{
			BatchSize = InputBatchSize;
			InputTensorShape = FTensorShape::Make({ BatchSize, InputSize });
			OutputTensorShape = FTensorShape::Make({ BatchSize, OutputSize });

			if (Instance)
			{
				Instance->SetMaxBatchSize(BatchSize);
			}
		}

		return ESetInputTensorShapesStatus::Ok;
	}

	FModelInstanceCPU::ERunSyncStatus FModelInstanceCPU::RunSync(TConstArrayView<FTensorBindingCPU> InInputBindings, TConstArrayView<FTensorBindingCPU> InOutputBindings)
	{
		NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::FModelInstanceCPU::RunSync);

		if (!ensureMsgf(BatchSize > 0, TEXT("SetInputTensorShapes must be run before RunSync")))
		{
			return ERunSyncStatus::Fail;
		}

		if (!ensureMsgf(InInputBindings.Num() == 1, TEXT("Basic CPU Inference only supports single input tensor.")))
		{
			return ERunSyncStatus::Fail;
		}

		if (!ensureMsgf(InOutputBindings.Num() == 1, TEXT("Basic CPU Inference only supports single output tensor.")))
		{
			return ERunSyncStatus::Fail;
		}

		if (!ensureMsgf(InInputBindings[0].SizeInBytes == BatchSize * InputSize * sizeof(float), TEXT("Incorrect Input Tensor Size")))
		{
			return ERunSyncStatus::Fail;
		}

		if (!ensureMsgf(InOutputBindings[0].SizeInBytes == BatchSize * OutputSize * sizeof(float), TEXT("Incorrect Output Tensor Size")))
		{
			return ERunSyncStatus::Fail;
		}

		Model->Layer->Evaluate(
			Instance.Get(),
			(float*)InOutputBindings[0].Data,
			(const float*)InInputBindings[0].Data,
			BatchSize,
			OutputSize,
			InputSize,
			OutputSize,
			InputSize);

		return ERunSyncStatus::Ok;
	}

	uint32 FModelCPU::ModelMagicNumber = 0x0BA51C01;
	uint32 FModelCPU::ModelVersionNumber = 1;

	TSharedPtr<IModelInstanceCPU> FModelCPU::CreateModelInstanceCPU()
	{
		return MakeShared<FModelInstanceCPU>(WeakThis.Pin());
	}

	void FModelCPU::SerializationSize(uint64& InOutOffset) const
	{
		checkf(InOutOffset % 64 == 0,
			TEXT("Model must be aligned to 64 bytes because there must be no padding before magic number."));

		Private::Serialization::Size(InOutOffset, ModelMagicNumber);
		Private::Serialization::Size(InOutOffset, ModelVersionNumber);
		Private::Serialization::Size(InOutOffset, Layer);
	}

	bool FModelCPU::SerializationLoad(uint64& InOutOffset, TConstArrayView<uint8> Data)
	{
		NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::FModelCPU::SerializationLoad);

		checkf(InOutOffset % 64 == 0, 
			TEXT("Model must be aligned to 64 bytes because there must be no padding before magic number."));

		uint32 Magic = INDEX_NONE;
		Private::Serialization::Load(InOutOffset, Magic, Data);
		if (Magic != ModelMagicNumber)
		{
			UE_LOGF(LogNNERuntimeBasicCPU, Error, "Invalid Magic Number %i", Magic);
			return false;
		}

		uint32 Version = INDEX_NONE;
		Private::Serialization::Load(InOutOffset, Version, Data);
		if (Version != ModelVersionNumber)
		{
			UE_LOGF(LogNNERuntimeBasicCPU, Error, "Unsupported Version Number %i", Version);
			return false;
		}

		Private::Serialization::Load(InOutOffset, Layer, Data);

		return true;
	}

	void FModelCPU::SerializationSave(uint64& InOutOffset, TArrayView<uint8> Data) const
	{
		NNE_RUNTIME_BASIC_TRACE_SCOPE(NNE::RuntimeBasic::FModelCPU::SerializationSave);

		checkf(InOutOffset % 64 == 0,
			TEXT("Model must be aligned to 64 bytes because there must be no padding before magic number."));

		Private::Serialization::Save(InOutOffset, ModelMagicNumber, Data);
		Private::Serialization::Save(InOutOffset, ModelVersionNumber, Data);
		Private::Serialization::Save(InOutOffset, Layer, Data);
	}

	//--------------------------------------------------------------------------
	// Builder
	//--------------------------------------------------------------------------

	namespace Private
	{
		static inline uint32 RngInt(const uint32 State)
		{
			uint32 X = State ^ 0xb74eaecf;
			X = ((X >> 16) ^ X) * 0x45d9f3b;
			X = ((X >> 16) ^ X) * 0x45d9f3b;
			return (X >> 16) ^ X;
		}

		static inline float RngUniform(const uint32 State)
		{
			// Same approach as used in FRandomStream
			float Output;
			*((uint32*)(&Output)) = 0x3F800000U | (RngInt(State ^ 0x1c89a74a) >> 9);
			return Output - 1.0f;
		}

		static inline float RngGaussian(const uint32 State)
		{
			return FMath::Sqrt(-2.0f * FMath::Loge(FMath::Max(RngUniform(State ^ 0xe427d90b), UE_SMALL_NUMBER))) * FMath::Cos(RngUniform(State ^ 0xd5444566) * UE_TWO_PI);
		}

		static inline void RngUpdate(uint32& State)
		{
			State = RngInt(State ^ 0x0c32dd74);
		}

		static inline float RngClippedGaussian(const uint32 State, const float Clip = 10.0f)
		{
			return FMath::Clamp(RngGaussian(State), -Clip, Clip);
		}

	}

	FModelBuilder::FWeightInitializationSettings::FWeightInitializationSettings() = default;
	FModelBuilder::FLinearLayerSettings::FLinearLayerSettings() = default;

	FModelBuilder::FModelBuilder(int32 Seed) : Rng(Seed) {};

	FModelBuilderElement::FModelBuilderElement() = default;
	FModelBuilderElement::FModelBuilderElement(const TSharedPtr<Private::ILayer>& Ptr) : Layer(Ptr) {}
	FModelBuilderElement::~FModelBuilderElement() = default;

	int32 FModelBuilderElement::GetInputSize() const
	{
		check(Layer);
		return Layer->GetInputSize();
	}

	int32 FModelBuilderElement::GetOutputSize() const
	{
		check(Layer);
		return Layer->GetOutputSize();
	}

	FModelBuilderElement FModelBuilder::MakeLinear(
		const uint32 InputSize,
		const uint32 OutputSize,
		const TConstArrayView<float> Weights,
		const TConstArrayView<float> Biases)
	{
		check(Biases.Num() == OutputSize);
		check(Weights.Num() == InputSize * OutputSize);

		const TSharedPtr<Private::FLinearLayer> LinearLayer = MakeShared<Private::FLinearLayer>();
		LinearLayer->InputSize = InputSize;
		LinearLayer->OutputSize = OutputSize;
		LinearLayer->Biases = Biases;
		LinearLayer->Weights = Weights;

		return StaticCastSharedPtr<Private::ILayer>(LinearLayer);
	}

	FModelBuilderElement FModelBuilder::MakeCompressedLinear(
		const uint32 InputSize,
		const uint32 OutputSize,
		const TConstArrayView<uint16> Weights,
		const TConstArrayView<float> WeightOffsets,
		const TConstArrayView<float> WeightScales,
		const TConstArrayView<float> Biases)
	{
		check(Biases.Num() == OutputSize);
		check(Weights.Num() == InputSize * OutputSize);

		const TSharedPtr<Private::FCompressedLinearLayer> LinearLayer = MakeShared<Private::FCompressedLinearLayer>();
		LinearLayer->InputSize = InputSize;
		LinearLayer->OutputSize = OutputSize;
		LinearLayer->WeightOffsets = WeightOffsets;
		LinearLayer->WeightScales = WeightScales;
		LinearLayer->Biases = Biases;
		LinearLayer->Weights = Weights;

		return StaticCastSharedPtr<Private::ILayer>(LinearLayer);
	}

	FModelBuilderElement FModelBuilder::MakeLipschiztLinear(
		const uint32 InputSize,
		const uint32 OutputSize,
		const TConstArrayView<float> Weights,
		const TConstArrayView<float> Biases)
	{
		check(Biases.Num() == OutputSize);
		check(Weights.Num() == InputSize * OutputSize);

		const TSharedPtr<Private::FLipschiztLinearLayer> LipschiztLinearLayer = MakeShared<Private::FLipschiztLinearLayer>();
		LipschiztLinearLayer->InputSize = InputSize;
		LipschiztLinearLayer->OutputSize = OutputSize;
		LipschiztLinearLayer->Biases = Biases;
		LipschiztLinearLayer->Weights = Weights;

		return StaticCastSharedPtr<Private::ILayer>(LipschiztLinearLayer);
	}

	FModelBuilderElement FModelBuilder::MakeLinearLayer(
		const uint32 InputSize,
		const uint32 OutputSize,
		const FLinearLayerSettings& Settings)
	{
		switch (Settings.Type)
		{
		case ELinearLayerType::Normal:
		{
			return MakeLinear(
				InputSize,
				OutputSize,
				MakeInitialWeights(InputSize, OutputSize, Settings.WeightInitializationSettings),
				MakeInitialBiases(OutputSize, Settings.WeightInitializationSettings));
		}
					
		case ELinearLayerType::Compressed:
		{
			TArrayView<uint16> Weights;
			TArrayView<float> WeightOffsets;
			TArrayView<float> WeightScales;

			MakeInitialCompressedWeights(
				Weights,
				WeightOffsets,
				WeightScales,
				InputSize,
				OutputSize,
				Settings.WeightInitializationSettings);

			return MakeCompressedLinear(
				InputSize,
				OutputSize,
				Weights,
				WeightOffsets,
				WeightScales,
				MakeInitialBiases(OutputSize, Settings.WeightInitializationSettings));
		}

		case ELinearLayerType::Lipschizt:
		{
			return MakeLipschiztLinear(
				InputSize,
				OutputSize,
				MakeInitialWeights(InputSize, OutputSize, Settings.WeightInitializationSettings),
				MakeInitialBiases(OutputSize, Settings.WeightInitializationSettings));
		}

		default:
		{
			checkNoEntry();
			return FModelBuilderElement();
		}

		}
	}

	FModelBuilderElement FModelBuilder::MakeMultiLinear(
		const uint32 InputSize, 
		const uint32 OutputSize, 
		const uint32 BlockNum, 
		const TConstArrayView<float> Weights, 
		const TConstArrayView<float> Biases)
	{
		check(Biases.Num() == OutputSize * BlockNum);
		check(Weights.Num() == InputSize * OutputSize * BlockNum);

		const TSharedPtr<Private::FMultiLinearLayer> MultiLinearLayer = MakeShared<Private::FMultiLinearLayer>();
		MultiLinearLayer->InputSize = InputSize;
		MultiLinearLayer->OutputSize = OutputSize;
		MultiLinearLayer->BlockNum = BlockNum;
		MultiLinearLayer->Biases = Biases;
		MultiLinearLayer->Weights = Weights;

		return StaticCastSharedPtr<Private::ILayer>(MultiLinearLayer);
	}

	FModelBuilderElement FModelBuilder::MakeNormalize(
		const uint32 InputOutputSize,
		const TConstArrayView<float> Mean,
		const TConstArrayView<float> Std)
	{
		check(Mean.Num() == InputOutputSize);
		check(Std.Num() == InputOutputSize);

		const TSharedPtr<Private::FNormalizeLayer> NormalizeLayer = MakeShared<Private::FNormalizeLayer>();
		NormalizeLayer->InputOutputSize = InputOutputSize;
		NormalizeLayer->Mean = Mean;
		NormalizeLayer->Std = Std;

		return StaticCastSharedPtr<Private::ILayer>(NormalizeLayer);
	}

	FModelBuilderElement FModelBuilder::MakeDenormalize(
		const uint32 InputOutputSize,
		const TConstArrayView<float> Mean,
		const TConstArrayView<float> Std)
	{
		check(Mean.Num() == InputOutputSize);
		check(Std.Num() == InputOutputSize);

		const TSharedPtr<Private::FDenormalizeLayer> DenormalizeLayer = MakeShared<Private::FDenormalizeLayer>();
		DenormalizeLayer->InputOutputSize = InputOutputSize;
		DenormalizeLayer->Mean = Mean;
		DenormalizeLayer->Std = Std;

		return StaticCastSharedPtr<Private::ILayer>(DenormalizeLayer);
	}

	FModelBuilderElement FModelBuilder::MakeReLU(const uint32 InputOutputSize)
	{
		const TSharedPtr<Private::FReLULayer> Layer = MakeShared<Private::FReLULayer>();
		Layer->InputOutputSize = InputOutputSize;
		return StaticCastSharedPtr<Private::ILayer>(Layer);
	}

	FModelBuilderElement FModelBuilder::MakeELU(const uint32 InputOutputSize)
	{
		const TSharedPtr<Private::FELULayer> Layer = MakeShared<Private::FELULayer>();
		Layer->InputOutputSize = InputOutputSize;
		return StaticCastSharedPtr<Private::ILayer>(Layer);
	}

	FModelBuilderElement FModelBuilder::MakeGELU(const uint32 InputOutputSize)
	{
		const TSharedPtr<Private::FGELULayer> Layer = MakeShared<Private::FGELULayer>();
		Layer->InputOutputSize = InputOutputSize;
		return StaticCastSharedPtr<Private::ILayer>(Layer);
	}

	FModelBuilderElement FModelBuilder::MakeTanH(const uint32 InputOutputSize)
	{
		const TSharedPtr<Private::FTanHLayer> Layer = MakeShared<Private::FTanHLayer>();
		Layer->InputOutputSize = InputOutputSize;
		return StaticCastSharedPtr<Private::ILayer>(Layer);
	}

	FModelBuilderElement FModelBuilder::MakeSoftmax(const uint32 InputOutputSize)
	{
		const TSharedPtr<Private::FSoftmaxLayer> Layer = MakeShared<Private::FSoftmaxLayer>();
		Layer->InputOutputSize = InputOutputSize;
		return StaticCastSharedPtr<Private::ILayer>(Layer);
	}

	FModelBuilderElement FModelBuilder::MakeCopy(const uint32 InputOutputSize)
	{
		const TSharedPtr<Private::FCopyLayer> Layer = MakeShared<Private::FCopyLayer>();
		Layer->InputOutputSize = InputOutputSize;
		return StaticCastSharedPtr<Private::ILayer>(Layer);
	}

	FModelBuilderElement FModelBuilder::MakeSlice(const uint32 InputSize, const uint32 SliceOffset, const uint32 SliceSize)
	{
		check(SliceOffset + SliceSize <= InputSize);

		const TSharedPtr<Private::FSliceLayer> Layer = MakeShared<Private::FSliceLayer>();
		Layer->InputSize = InputSize;
		Layer->SliceOffset = SliceOffset;
		Layer->SliceSize = SliceSize;
		return StaticCastSharedPtr<Private::ILayer>(Layer);
	}

	FModelBuilderElement FModelBuilder::MakeClamp(const uint32 InputOutputSize, const TConstArrayView<float> MinValues, const TConstArrayView<float> MaxValues)
	{
		check(MinValues.Num() == InputOutputSize);
		check(MaxValues.Num() == InputOutputSize);

		const TSharedPtr<Private::FClampLayer> Layer = MakeShared<Private::FClampLayer>();
		Layer->InputOutputSize = InputOutputSize;
		Layer->MinValues = MinValues;
		Layer->MaxValues = MaxValues;
		return StaticCastSharedPtr<Private::ILayer>(Layer);
	}

	FModelBuilderElement FModelBuilder::MakeActivation(const uint32 InputOutputSize, const EActivationFunction ActivationFunction)
	{
		switch (ActivationFunction)
		{
		case EActivationFunction::ReLU: return MakeReLU(InputOutputSize);
		case EActivationFunction::ELU: return MakeELU(InputOutputSize);
		case EActivationFunction::TanH: return MakeTanH(InputOutputSize);
		case EActivationFunction::GELU: return MakeGELU(InputOutputSize);
		default:
			checkf(false, TEXT("Unknown Activation Function"));
			return MakeReLU(InputOutputSize);
		}
	}

	FModelBuilderElement FModelBuilder::MakePReLU(const uint32 InputOutputSize, TConstArrayView<float> Alpha)
	{
		check(Alpha.Num() == InputOutputSize);

		const TSharedPtr<Private::FPReLULayer> ActivationLayer = MakeShared<Private::FPReLULayer>();
		ActivationLayer->InputOutputSize = InputOutputSize;
		ActivationLayer->Alpha = Alpha;

		return StaticCastSharedPtr<Private::ILayer>(ActivationLayer);
	}

	FModelBuilderElement FModelBuilder::MakeSequence(const TConstArrayView<FModelBuilderElement> Elements)
	{
		const TSharedPtr<Private::FSequenceLayer> SequenceLayer = MakeShared<Private::FSequenceLayer>();
		SequenceLayer->Layers.Reserve(Elements.Num());

		for (const FModelBuilderElement& Element : Elements)
		{
			SequenceLayer->Layers.Emplace(Element.Layer);
		}

		for (int32 LayerIdx = 1; LayerIdx < Elements.Num(); LayerIdx++)
		{
			const int32 PrevLayerOutputSize = SequenceLayer->Layers[LayerIdx - 1]->GetOutputSize();
			const int32 NextLayerInputSize = SequenceLayer->Layers[LayerIdx - 0]->GetInputSize();
			checkf(PrevLayerOutputSize == NextLayerInputSize, TEXT("Sequence Layer Dimensions don't match. Output %i vs Input %i."), PrevLayerOutputSize, NextLayerInputSize);
		}

		return StaticCastSharedPtr<Private::ILayer>(SequenceLayer);
	}

	FModelBuilderElement FModelBuilder::MakeMLP(
		const uint32 InputSize,
		const uint32 OutputSize,
		const uint32 HiddenSize,
		const uint32 LayerNum,
		const EActivationFunction ActivationFunction,
		const bool bActivationOnFinalLayer,
		const FLinearLayerSettings& LinearLayerSettings)
	{
		check(LayerNum >= 1);

		// Reserve space for Linear + Activation on each layer, minus the final activation if it is not required.
		const int32 TotalLayerNum = 2 * LayerNum - (bActivationOnFinalLayer ? 0 : 1);

		TArray<FModelBuilderElement, TInlineAllocator<32>> Layers;
		Layers.Reserve(TotalLayerNum);

		for (uint32 LayerIdx = 0; LayerIdx < LayerNum; LayerIdx++)
		{
			const uint32 LayerInputSize = LayerIdx == 0 ? InputSize : HiddenSize;
			const uint32 LayerOuputSize = LayerIdx == LayerNum - 1 ? OutputSize : HiddenSize;

			Layers.Emplace(MakeLinearLayer(LayerInputSize, LayerOuputSize, LinearLayerSettings));
			
			if (bActivationOnFinalLayer || LayerIdx != LayerNum - 1)
			{
				Layers.Emplace(MakeActivation(LayerOuputSize, ActivationFunction));
			}
		}
		
		check(Layers.Num() == TotalLayerNum);

		return MakeSequence(Layers);
	}

	FModelBuilderElement FModelBuilder::MakeMLPWithLayerNorm(
		const uint32 InputSize, 
		const uint32 OutputSize, 
		const uint32 HiddenSize, 
		const uint32 LayerNum, 
		const EActivationFunction ActivationFunction, 
		const bool bActivationOnFinalLayer,
		const FLinearLayerSettings& LinearLayerSettings)
	{
		check(LayerNum >= 1);

		// Reserve space for Linear + Activation on each layer, minus the final activation if it is not required, 
		// plus the LayerNorm in between each layer.
		const int32 TotalLayerNum = 2 * LayerNum - (bActivationOnFinalLayer ? 0 : 1) + LayerNum - 1;

		TArray<FModelBuilderElement, TInlineAllocator<32>> Layers;
		Layers.Reserve(TotalLayerNum);

		for (uint32 LayerIdx = 0; LayerIdx < LayerNum; LayerIdx++)
		{
			const uint32 LayerInputSize = LayerIdx == 0 ? InputSize : HiddenSize;
			const uint32 LayerOuputSize = LayerIdx == LayerNum - 1 ? OutputSize : HiddenSize;

			Layers.Emplace(MakeLinearLayer(LayerInputSize, LayerOuputSize, LinearLayerSettings));
			
			if (LayerIdx != LayerNum - 1)
			{
				Layers.Emplace(MakeLayerNorm(
					LayerOuputSize, 
					MakeValuesZero(LayerOuputSize), 
					MakeValuesOne(LayerOuputSize)));
			}

			if (bActivationOnFinalLayer || LayerIdx != LayerNum - 1)
			{
				Layers.Emplace(MakeActivation(LayerOuputSize, ActivationFunction));
			}
		}

		check(Layers.Num() == TotalLayerNum);

		return MakeSequence(Layers);
	}

	FModelBuilderElement FModelBuilder::MakeSkipMLP(
		const uint32 InputSize,
		const uint32 OutputSize,
		const uint32 HiddenSize,
		const uint32 LayerNum,
		const EActivationFunction ActivationFunction,
		const bool bActivationOnFinalLayer,
		const FLinearLayerSettings& LinearLayerSettings)
	{
		check(LayerNum >= 2);

		const int32 TotalLayerNum = LayerNum;

		TArray<FModelBuilderElement, TInlineAllocator<32>> Layers;
		Layers.Reserve(TotalLayerNum);

		for (uint32 LayerIdx = 0; LayerIdx < LayerNum; LayerIdx++)
		{
			if (LayerIdx == 0)
			{
				Layers.Emplace(MakeSpread({
					MakeCopy(InputSize),
					MakeSequence({
						MakeLinearLayer(InputSize, HiddenSize, LinearLayerSettings),
						MakeActivation(HiddenSize, ActivationFunction),
					})
					}));
			}
			else if (LayerIdx != LayerNum - 1)
			{
				Layers.Emplace(MakeSpread({
					MakeSlice(InputSize + HiddenSize, 0, InputSize),
					MakeSequence({
						MakeLinearLayer(InputSize + HiddenSize, HiddenSize, LinearLayerSettings),
						MakeActivation(HiddenSize, ActivationFunction),
					})
					}));
			}
			else
			{
				if (bActivationOnFinalLayer)
				{
					Layers.Emplace(MakeSequence({
							MakeLinearLayer(InputSize + HiddenSize, OutputSize, LinearLayerSettings),
							MakeActivation(OutputSize, ActivationFunction),
						}));
				}
				else
				{
					Layers.Emplace(MakeLinearLayer(InputSize + HiddenSize, OutputSize, LinearLayerSettings));
				}
			}
		}

		check(Layers.Num() == TotalLayerNum);

		return MakeSequence(Layers);
	}
	
	FModelBuilderElement FModelBuilder::MakeSkipMLPWithLayerNorm(
		const uint32 InputSize,
		const uint32 OutputSize,
		const uint32 HiddenSize,
		const uint32 LayerNum,
		const EActivationFunction ActivationFunction,
		const bool bActivationOnFinalLayer,
		const FLinearLayerSettings& LinearLayerSettings)
	{
		check(LayerNum >= 2);

		const int32 TotalLayerNum = LayerNum;

		TArray<FModelBuilderElement, TInlineAllocator<32>> Layers;
		Layers.Reserve(TotalLayerNum);

		for (uint32 LayerIdx = 0; LayerIdx < LayerNum; LayerIdx++)
		{
			if (LayerIdx == 0)
			{
				Layers.Emplace(MakeSpread({
					MakeCopy(InputSize),
					MakeLinearLayer(InputSize, HiddenSize, LinearLayerSettings)
					}));
			}
			else if (LayerIdx != LayerNum - 1)
			{
				Layers.Emplace(MakeSpread({
					MakeSlice(InputSize + HiddenSize, 0, InputSize),
					MakeSequence({
						MakeLinearLayer(InputSize + HiddenSize, HiddenSize, LinearLayerSettings),
						MakeLayerNorm(HiddenSize, MakeValuesZero(HiddenSize), MakeValuesOne(HiddenSize)),
						MakeActivation(HiddenSize, ActivationFunction),
					})
					}));
			}
			else
			{
				if (bActivationOnFinalLayer)
				{
					Layers.Emplace(MakeSequence({
						MakeLinearLayer(InputSize + HiddenSize, OutputSize, LinearLayerSettings),
						MakeActivation(OutputSize, ActivationFunction) }));
				}
				else
				{
					Layers.Emplace(MakeLinearLayer(InputSize + HiddenSize, OutputSize, LinearLayerSettings));
				}
			}
		}

		check(Layers.Num() == TotalLayerNum);

		return MakeSequence(Layers);
	}

	FModelBuilderElement FModelBuilder::MakeResidualMLP(
		const uint32 InputSize,
		const uint32 OutputSize,
		const uint32 HiddenSize,
		const uint32 LayerNum,
		const EActivationFunction ActivationFunction,
		const bool bActivationOnFinalLayer,
		const FLinearLayerSettings& LinearLayerSettings)
	{
		check(LayerNum >= 2);

		const int32 TotalLayerNum = LayerNum;

		TArray<FModelBuilderElement, TInlineAllocator<32>> Layers;
		Layers.Reserve(TotalLayerNum);

		for (uint32 LayerIdx = 0; LayerIdx < LayerNum; LayerIdx++)
		{
			if (LayerIdx == 0)
			{
				Layers.Emplace(
					MakeSequence({
						MakeLinearLayer(InputSize, HiddenSize, LinearLayerSettings),
						MakeActivation(HiddenSize, ActivationFunction),
					}));
			}
			else if (LayerIdx != LayerNum - 1)
			{
				Layers.Emplace(MakeResidual(
					MakeSequence({
						MakeLinearLayer(HiddenSize, HiddenSize, LinearLayerSettings),
						MakeActivation(HiddenSize, ActivationFunction),
					})));
			}
			else
			{
				if (bActivationOnFinalLayer)
				{
					Layers.Emplace(MakeSequence({
							MakeLinearLayer(HiddenSize, OutputSize, LinearLayerSettings),
							MakeActivation(OutputSize, ActivationFunction),
						}));
				}
				else
				{
					Layers.Emplace(MakeLinearLayer(HiddenSize, OutputSize, LinearLayerSettings));
				}
			}
		}

		check(Layers.Num() == TotalLayerNum);

		return MakeSequence(Layers);
	}

	FModelBuilderElement FModelBuilder::MakeResidualMLPWithLayerNorm(
		const uint32 InputSize,
		const uint32 OutputSize,
		const uint32 HiddenSize,
		const uint32 LayerNum,
		const EActivationFunction ActivationFunction,
		const bool bActivationOnFinalLayer,
		const FLinearLayerSettings& LinearLayerSettings)
	{
		check(LayerNum >= 2);

		const int32 TotalLayerNum = LayerNum;

		TArray<FModelBuilderElement, TInlineAllocator<32>> Layers;
		Layers.Reserve(TotalLayerNum);

		for (uint32 LayerIdx = 0; LayerIdx < LayerNum; LayerIdx++)
		{
			if (LayerIdx == 0)
			{
				Layers.Emplace(
					MakeSequence({
						MakeLinearLayer(InputSize, HiddenSize, LinearLayerSettings),
						MakeActivation(HiddenSize, ActivationFunction),
						}));
			}
			else if (LayerIdx != LayerNum - 1)
			{
				Layers.Emplace(MakeResidual(
					MakeSequence({
						MakeLinearLayer(HiddenSize, HiddenSize, LinearLayerSettings),
						MakeLayerNorm(HiddenSize, MakeValuesZero(HiddenSize), MakeValuesOne(HiddenSize)),
						MakeActivation(HiddenSize, ActivationFunction),
						})));
			}
			else
			{
				if (bActivationOnFinalLayer)
				{
					Layers.Emplace(MakeSequence({
						MakeLinearLayer(HiddenSize, OutputSize, LinearLayerSettings),
						MakeActivation(OutputSize, ActivationFunction) }));
				}
				else
				{
					Layers.Emplace(MakeLinearLayer(HiddenSize, OutputSize, LinearLayerSettings));
				}
			}
		}

		check(Layers.Num() == TotalLayerNum);

		return MakeSequence(Layers);
	}

	FModelBuilderElement FModelBuilder::MakeMemoryCell(
		const uint32 InputNum,
		const uint32 OutputNum,
		const uint32 MemoryNum,
		const FModelBuilderElement& RememberLayer,
		const FModelBuilderElement& PassthroughLayer,
		const FModelBuilderElement& MemoryUpdateLayer,
		const FModelBuilderElement& OutputInputUpdateLayer,
		const FModelBuilderElement& OutputMemoryUpdateLayer)
	{
		check(RememberLayer.GetInputSize() == InputNum + MemoryNum);
		check(RememberLayer.GetOutputSize() == MemoryNum);
		check(PassthroughLayer.GetInputSize() == InputNum + MemoryNum);
		check(PassthroughLayer.GetOutputSize() == OutputNum);
		check(MemoryUpdateLayer.GetInputSize() == InputNum + MemoryNum);
		check(MemoryUpdateLayer.GetOutputSize() == MemoryNum);
		check(OutputInputUpdateLayer.GetInputSize() == InputNum + MemoryNum);
		check(OutputInputUpdateLayer.GetOutputSize() == OutputNum);
		check(OutputMemoryUpdateLayer.GetInputSize() == MemoryNum);
		check(OutputMemoryUpdateLayer.GetOutputSize() == OutputNum);

		TSharedPtr<Private::FMemoryCellLayer> CellLayer = MakeShared<Private::FMemoryCellLayer>();
		CellLayer->InputSize = InputNum;
		CellLayer->OutputSize = OutputNum;
		CellLayer->MemorySize = MemoryNum;
		CellLayer->RememberLayer = RememberLayer.Layer;
		CellLayer->PassthroughLayer = PassthroughLayer.Layer;
		CellLayer->MemoryUpdateLayer = MemoryUpdateLayer.Layer;
		CellLayer->OutputInputUpdateLayer = OutputInputUpdateLayer.Layer;
		CellLayer->OutputMemoryUpdateLayer = OutputMemoryUpdateLayer.Layer;
		
		return StaticCastSharedPtr<Private::ILayer>(CellLayer);
	}

	FModelBuilderElement FModelBuilder::MakeMemoryCellLayer(
		const uint32 InputNum,
		const uint32 OutputNum,
		const uint32 MemoryNum,
		const FLinearLayerSettings& LinearLayerSettings)
	{
		return MakeMemoryCell(
			InputNum,
			OutputNum,
			MemoryNum,
			MakeLinearLayer(InputNum + MemoryNum, MemoryNum, LinearLayerSettings),
			MakeLinearLayer(InputNum + MemoryNum, OutputNum, LinearLayerSettings),
			MakeLinearLayer(InputNum + MemoryNum, MemoryNum, LinearLayerSettings),
			MakeLinearLayer(InputNum + MemoryNum, OutputNum, LinearLayerSettings),
			MakeLinearLayer(MemoryNum, OutputNum, LinearLayerSettings));
	}


	FModelBuilderElement FModelBuilder::MakeGRUCell(
		const uint32 InputNum,
		const uint32 OutputNum,
		const FModelBuilderElement& RememberLayer,
		const FModelBuilderElement& UpdateLayer,
		const FModelBuilderElement& ActivationLayer)
	{
		check(RememberLayer.GetInputSize() == InputNum + OutputNum);
		check(RememberLayer.GetOutputSize() == OutputNum);
		check(UpdateLayer.GetInputSize() == InputNum + OutputNum);
		check(UpdateLayer.GetOutputSize() == OutputNum);
		check(ActivationLayer.GetInputSize() == InputNum + OutputNum);
		check(ActivationLayer.GetOutputSize() == OutputNum);

		TSharedPtr<Private::FGRUCellLayer> CellLayer = MakeShared<Private::FGRUCellLayer>();
		CellLayer->InputSize = InputNum;
		CellLayer->OutputSize = OutputNum;
		CellLayer->RememberLayer = RememberLayer.Layer;
		CellLayer->UpdateLayer = UpdateLayer.Layer;
		CellLayer->ActivationLayer = ActivationLayer.Layer;

		return StaticCastSharedPtr<Private::ILayer>(CellLayer);
	}

	FModelBuilderElement FModelBuilder::MakeGRUCellLayer(
		const uint32 InputNum,
		const uint32 OutputNum,
		const FLinearLayerSettings& LinearLayerSettings)
	{
		return MakeGRUCell(
			InputNum,
			OutputNum,
			MakeLinearLayer(InputNum + OutputNum, OutputNum, LinearLayerSettings),
			MakeLinearLayer(InputNum + OutputNum, OutputNum, LinearLayerSettings),
			MakeLinearLayer(InputNum + OutputNum, OutputNum, LinearLayerSettings));
	}

	FModelBuilderElement FModelBuilder::MakeMemoryBackbone(
		const uint32 MemoryNum,
		const FModelBuilderElement& Prefix,
		const FModelBuilderElement& Cell,
		const FModelBuilderElement& Postfix)
	{
		check(Prefix.GetOutputSize() == Cell.GetInputSize() - MemoryNum);
		check(Postfix.GetInputSize() == Cell.GetOutputSize() - MemoryNum);

		return MakeSequence({
			MakeConcat({
				Prefix,
				MakeCopy(MemoryNum)
				}),
			Cell,
			MakeConcat({
				Postfix,
				MakeCopy(MemoryNum)
			})
		});
	}

	FModelBuilderElement FModelBuilder::MakeConcat(const TConstArrayView<FModelBuilderElement> Elements)
	{
		const int32 LayerNum = Elements.Num();

		const TSharedPtr<Private::FConcatLayer> ConcatLayer = MakeShared<Private::FConcatLayer>();
		ConcatLayer->InputSizes = MakeSizesLayerInputs(Elements);
		ConcatLayer->OutputSizes = MakeSizesLayerOutputs(Elements);
		ConcatLayer->Layers.Reserve(LayerNum);

		for (const FModelBuilderElement& Element : Elements)
		{
			ConcatLayer->Layers.Emplace(Element.Layer);
		}

		ConcatLayer->PostLoad();

		return StaticCastSharedPtr<Private::ILayer>(ConcatLayer);
	}

	FModelBuilderElement FModelBuilder::MakeSpread(const TConstArrayView<FModelBuilderElement> Elements)
	{
		const int32 LayerNum = Elements.Num();

		const TSharedPtr<Private::FSpreadLayer> SpreadLayer = MakeShared<Private::FSpreadLayer>();
		SpreadLayer->InputSize = LayerNum ? Elements[0].GetInputSize() : 0;
		SpreadLayer->OutputSizes = MakeSizesLayerOutputs(Elements);
		SpreadLayer->Layers.Reserve(LayerNum);

		for (const FModelBuilderElement& Element : Elements)
		{
			check(SpreadLayer->InputSize == Element.GetInputSize());
			SpreadLayer->Layers.Emplace(Element.Layer);
		}

		SpreadLayer->PostLoad();

		return StaticCastSharedPtr<Private::ILayer>(SpreadLayer);
	}

	FModelBuilderElement FModelBuilder::MakeArray(const uint32 ElementNum, const FModelBuilderElement& SubLayer)
	{
		const TSharedPtr<Private::FArrayLayer> ArrayLayer = MakeShared<Private::FArrayLayer>();
		ArrayLayer->ElementNum = ElementNum;
		ArrayLayer->ElementInputSize = SubLayer.GetInputSize();
		ArrayLayer->ElementOutputSize = SubLayer.GetOutputSize();
		ArrayLayer->SubLayer = SubLayer.Layer;

		return StaticCastSharedPtr<Private::ILayer>(ArrayLayer);
	}

	FModelBuilderElement FModelBuilder::MakeResidual(const FModelBuilderElement& SubLayer)
	{
		check(SubLayer.GetInputSize() == SubLayer.GetOutputSize())

		const TSharedPtr<Private::FResidualLayer> ResidualLayer = MakeShared<Private::FResidualLayer>();
		ResidualLayer->InputOutputSize = SubLayer.GetInputSize();
		ResidualLayer->SubLayer = SubLayer.Layer;

		return StaticCastSharedPtr<Private::ILayer>(ResidualLayer);
	}

	FModelBuilderElement FModelBuilder::MakeAggregateSet(
		const uint32 MaxElementNum,
		const uint32 OutputEncodingSize,
		const uint32 AttentionEncodingSize,
		const uint32 AttentionHeadNum,
		const FModelBuilderElement& SubLayer,
		const FModelBuilderElement& QueryLayer,
		const FModelBuilderElement& KeyLayer,
		const FModelBuilderElement& ValueLayer)
	{
		check(SubLayer.GetOutputSize() == QueryLayer.GetInputSize());
		check(SubLayer.GetOutputSize() == KeyLayer.GetInputSize());
		check(SubLayer.GetOutputSize() == ValueLayer.GetInputSize());
		check(QueryLayer.GetOutputSize() == AttentionHeadNum * AttentionEncodingSize);
		check(KeyLayer.GetOutputSize() == AttentionHeadNum * AttentionEncodingSize);
		check(ValueLayer.GetOutputSize() == AttentionHeadNum * OutputEncodingSize);

		const TSharedPtr<Private::FAggregateSetLayer> SetLayer = MakeShared<Private::FAggregateSetLayer>();
		SetLayer->MaxElementNum = MaxElementNum;
		SetLayer->ElementInputSize = SubLayer.GetInputSize();
		SetLayer->ElementOutputSize = SubLayer.GetOutputSize();
		SetLayer->OutputEncodingSize = OutputEncodingSize;
		SetLayer->AttentionEncodingSize = AttentionEncodingSize;
		SetLayer->AttentionHeadNum = AttentionHeadNum;
		SetLayer->SubLayer = SubLayer.Layer;
		SetLayer->QueryLayer = QueryLayer.Layer;
		SetLayer->KeyLayer = KeyLayer.Layer;
		SetLayer->ValueLayer = ValueLayer.Layer;

		return StaticCastSharedPtr<Private::ILayer>(SetLayer);
	}

	FModelBuilderElement FModelBuilder::MakeAggregateOrExclusive(
		const uint32 OutputEncodingSize,
		const TConstArrayView<FModelBuilderElement> SubLayers,
		const TConstArrayView<FModelBuilderElement> Encoders)
	{
		check(SubLayers.Num() == Encoders.Num());

		const int32 SubLayerNum = SubLayers.Num();
		for (int32 SubLayerIdx = 0; SubLayerIdx < SubLayerNum; SubLayerIdx++)
		{
			check(SubLayers[SubLayerIdx].GetOutputSize() == Encoders[SubLayerIdx].GetInputSize());
			check(Encoders[SubLayerIdx].GetOutputSize() == OutputEncodingSize);
		}

		const TSharedPtr<Private::FAggregateOrExclusiveLayer> OrExclusiveLayer = MakeShared<Private::FAggregateOrExclusiveLayer>();
		OrExclusiveLayer->OutputEncodingSize = OutputEncodingSize;
		OrExclusiveLayer->SubLayerInputSizes = MakeSizesLayerInputs(SubLayers);
		OrExclusiveLayer->SubLayerOutputSizes = MakeSizesLayerOutputs(SubLayers);
		OrExclusiveLayer->SubLayers.Reserve(SubLayerNum);
		OrExclusiveLayer->Encoders.Reserve(SubLayerNum);

		for (int32 SubLayerIdx = 0; SubLayerIdx < SubLayerNum; SubLayerIdx++)
		{
			OrExclusiveLayer->SubLayers.Add(SubLayers[SubLayerIdx].Layer);
			OrExclusiveLayer->Encoders.Add(Encoders[SubLayerIdx].Layer);
		}

		OrExclusiveLayer->PostLoad();

		return StaticCastSharedPtr<Private::ILayer>(OrExclusiveLayer);
	}

	FModelBuilderElement FModelBuilder::MakeAggregateOrInclusive(
		const uint32 OutputEncodingSize,
		const uint32 AttentionEncodingSize,
		const uint32 AttentionHeadNum,
		const TConstArrayView<FModelBuilderElement> SubLayers,
		const TConstArrayView<FModelBuilderElement> QueryLayers,
		const TConstArrayView<FModelBuilderElement> KeyLayers,
		const TConstArrayView<FModelBuilderElement> ValueLayers)
	{
		check(SubLayers.Num() == QueryLayers.Num());
		check(SubLayers.Num() == KeyLayers.Num());
		check(SubLayers.Num() == ValueLayers.Num());

		const int32 SubLayerNum = SubLayers.Num();
		for (int32 SubLayerIdx = 0; SubLayerIdx < SubLayerNum; SubLayerIdx++)
		{
			check(SubLayers[SubLayerIdx].GetOutputSize() == QueryLayers[SubLayerIdx].GetInputSize());
			check(SubLayers[SubLayerIdx].GetOutputSize() == KeyLayers[SubLayerIdx].GetInputSize());
			check(SubLayers[SubLayerIdx].GetOutputSize() == ValueLayers[SubLayerIdx].GetInputSize());
			check(QueryLayers[SubLayerIdx].GetOutputSize() == AttentionHeadNum * AttentionEncodingSize);
			check(KeyLayers[SubLayerIdx].GetOutputSize() == AttentionHeadNum * AttentionEncodingSize);
			check(ValueLayers[SubLayerIdx].GetOutputSize() == AttentionHeadNum * OutputEncodingSize);
		}

		const TSharedPtr<Private::FAggregateOrInclusiveLayer> OrInclusiveLayer = MakeShared<Private::FAggregateOrInclusiveLayer>();
		OrInclusiveLayer->OutputEncodingSize = OutputEncodingSize;
		OrInclusiveLayer->AttentionEncodingSize = AttentionEncodingSize;
		OrInclusiveLayer->AttentionHeadNum = AttentionHeadNum;
		OrInclusiveLayer->SubLayerInputSizes = MakeSizesLayerInputs(SubLayers);
		OrInclusiveLayer->SubLayerOutputSizes = MakeSizesLayerOutputs(SubLayers);
		OrInclusiveLayer->SubLayers.Reserve(SubLayerNum);
		OrInclusiveLayer->QueryLayers.Reserve(SubLayerNum);
		OrInclusiveLayer->KeyLayers.Reserve(SubLayerNum);
		OrInclusiveLayer->ValueLayers.Reserve(SubLayerNum);

		for (int32 SubLayerIdx = 0; SubLayerIdx < SubLayerNum; SubLayerIdx++)
		{
			OrInclusiveLayer->SubLayers.Add(SubLayers[SubLayerIdx].Layer);
			OrInclusiveLayer->QueryLayers.Add(QueryLayers[SubLayerIdx].Layer);
			OrInclusiveLayer->KeyLayers.Add(KeyLayers[SubLayerIdx].Layer);
			OrInclusiveLayer->ValueLayers.Add(ValueLayers[SubLayerIdx].Layer);
		}

		OrInclusiveLayer->PostLoad();

		return StaticCastSharedPtr<Private::ILayer>(OrInclusiveLayer);
	}

	FModelBuilderElement FModelBuilder::MakeSparseMixtureOfExperts(
		const uint32 InputNum,
		const uint32 OutputNum,
		const FModelBuilderElement& GatingLayer,
		const TConstArrayView<FModelBuilderElement> SubLayers)
	{
		check(GatingLayer.GetInputSize() == InputNum);
		check(GatingLayer.GetOutputSize() == SubLayers.Num());

		const int32 SubLayerNum = SubLayers.Num();
		for (int32 SubLayerIdx = 0; SubLayerIdx < SubLayerNum; SubLayerIdx++)
		{
			check(SubLayers[SubLayerIdx].GetInputSize() == InputNum);
			check(SubLayers[SubLayerIdx].GetOutputSize() == OutputNum);
		}

		const TSharedPtr<Private::FSparseMixtureOfExpertsLayer> SparseMixtureOfExpertsLayer = MakeShared<Private::FSparseMixtureOfExpertsLayer>();
		SparseMixtureOfExpertsLayer->InputSize = InputNum;
		SparseMixtureOfExpertsLayer->OutputSize = OutputNum;
		SparseMixtureOfExpertsLayer->GatingLayer = GatingLayer.Layer;
		SparseMixtureOfExpertsLayer->SubLayers.Reserve(SubLayers.Num());

		for (int32 SubLayerIdx = 0; SubLayerIdx < SubLayerNum; SubLayerIdx++)
		{
			SparseMixtureOfExpertsLayer->SubLayers.Add(SubLayers[SubLayerIdx].Layer);
		}

		return StaticCastSharedPtr<Private::ILayer>(SparseMixtureOfExpertsLayer);
	}

	FModelBuilderElement FModelBuilder::MakeLayerNorm(
		const uint32 InputOutputSize,
		const TConstArrayView<float> Offsets,
		const TConstArrayView<float> Scales,
		const float Epsilon)
	{
		check(Offsets.Num() == InputOutputSize);
		check(Scales.Num() == InputOutputSize);

		const TSharedPtr<Private::FLayerNormLayer> LayerNormLayer = MakeShared<Private::FLayerNormLayer>();
		LayerNormLayer->InputOutputSize = InputOutputSize;
		LayerNormLayer->Offset = Offsets;
		LayerNormLayer->Scale = Scales;
		LayerNormLayer->Epsilon = Epsilon;

		return StaticCastSharedPtr<Private::ILayer>(LayerNormLayer);
	}

	FModelBuilderElement FModelBuilder::MakeTile(const uint32 InputSize, const uint32 Repeats)
	{
		const TSharedPtr<Private::FTileLayer> Layer = MakeShared<Private::FTileLayer>();
		Layer->InputSize = InputSize;
		Layer->Repeats = Repeats;
		return StaticCastSharedPtr<Private::ILayer>(Layer);
	}

	FModelBuilderElement FModelBuilder::MakeFiLMNetwork(
		const FModelBuilderElement& Prefix,
		const FModelBuilderElement& Condition,
		const FModelBuilderElement& Postfix)
	{
		check(Prefix.GetOutputSize() * 2 == Condition.GetOutputSize());
		check(Prefix.GetOutputSize() == Postfix.GetInputSize());

		const TSharedPtr<Private::FFiLMLayer> Layer = MakeShared<Private::FFiLMLayer>();
		Layer->PrefixInputSize = Prefix.GetInputSize();
		Layer->PrefixOutputSize = Prefix.GetOutputSize();
		Layer->ConditionInputSize = Condition.GetInputSize();
		Layer->ConditionOutputSize = Condition.GetOutputSize();
		Layer->PostfixInputSize = Postfix.GetInputSize();
		Layer->PostfixOutputSize = Postfix.GetOutputSize();
		Layer->PrefixLayer = Prefix.Layer;
		Layer->ConditionLayer = Condition.Layer;
		Layer->PostfixLayer = Postfix.Layer;
		return StaticCastSharedPtr<Private::ILayer>(Layer);
	}

	FModelBuilderElement FModelBuilder::MakeConv1d(
		const uint32 InputLength,
		const uint32 InChannels,
		const uint32 OutChannels,
		const uint32 KernelSize,
		const uint32 Padding,
		const EPaddingMode PaddingMode,
		const TConstArrayView<float> Weights,
		const TConstArrayView<float> Biases,
		const FModelBuilderElement& SubLayer)
	{
		check(Weights.Num() == InChannels * OutChannels * KernelSize);
		check(Biases.Num() == OutChannels);

		const TSharedPtr<Private::FConv1dLayer> Conv1dLayer = MakeShared<Private::FConv1dLayer>();
		Conv1dLayer->InputLength = InputLength;
		Conv1dLayer->InChannels = InChannels;
		Conv1dLayer->OutChannels = OutChannels;
		Conv1dLayer->KernelSize = KernelSize;
		Conv1dLayer->Padding = Padding;
		Conv1dLayer->PaddingMode = PaddingMode;
		Conv1dLayer->Weights = Weights;
		Conv1dLayer->Biases = Biases;
		Conv1dLayer->SubLayer = SubLayer.Layer;
		
		return StaticCastSharedPtr<Private::ILayer>(Conv1dLayer);
	}

	FModelBuilderElement FModelBuilder::MakeConv2d(
		const uint32 InputHeight,
		const uint32 InputWidth,
		const uint32 InChannels,
		const uint32 OutChannels,
		const uint32 KernelSize,
		const uint32 Stride,
		const uint32 Padding,
		const EPaddingMode PaddingMode,
		const TConstArrayView<float> Weights,
		const TConstArrayView<float> Biases,
		const FModelBuilderElement& SubLayer)
	{
		check(Weights.Num() == InChannels * OutChannels * KernelSize * KernelSize);
		check(Biases.Num() == OutChannels);

		const TSharedPtr<Private::FConv2dLayer> Conv2dLayer = MakeShared<Private::FConv2dLayer>();
		Conv2dLayer->InputHeight = InputHeight;
		Conv2dLayer->InputWidth = InputWidth;
		Conv2dLayer->InChannels = InChannels;
		Conv2dLayer->OutChannels = OutChannels;
		Conv2dLayer->KernelSize = KernelSize;
		Conv2dLayer->Stride = Stride;
		Conv2dLayer->Padding = Padding;
		Conv2dLayer->PaddingMode = PaddingMode;
		Conv2dLayer->Weights = Weights;
		Conv2dLayer->Biases = Biases;
		Conv2dLayer->SubLayer = SubLayer.Layer;

		return StaticCastSharedPtr<Private::ILayer>(Conv2dLayer);
	}

	void FModelBuilder::Reset()
	{
		Rng = RngInitialState;
		WeightsPool.Empty();
		CompressedWeightsPool.Empty();
		SizesPool.Empty();
	}

	uint64 FModelBuilder::GetWriteByteNum(const FModelBuilderElement& Element) const
	{
		FModelCPU Model;
		Model.Layer = Element.Layer;

		uint64 Offset = 0;
		Model.SerializationSize(Offset);
		return Offset;
	}

	void FModelBuilder::WriteFileData(TArrayView<uint8> OutBytes, uint32& OutInputSize, uint32& OutOutputSize, const FModelBuilderElement& Element) const
	{
		check((uint64)OutBytes.Num() == GetWriteByteNum(Element));

		OutInputSize = Element.GetInputSize();
		OutOutputSize = Element.GetOutputSize();

		// Zero to ensure any padding due to alignment is always zero
		FMemory::Memzero(OutBytes.GetData(), OutBytes.Num());

		FModelCPU Model;
		Model.Layer = Element.Layer;

		uint64 Offset = 0;
		Model.SerializationSave(Offset, OutBytes);
		check(Offset == OutBytes.Num());
	}

	void FModelBuilder::WriteFileData(TArray<uint8>& FileData, uint32& OutInputSize, uint32& OutOutputSize, const FModelBuilderElement& Element) const
	{
		FileData.SetNumUninitialized(GetWriteByteNum(Element));
		WriteFileData(MakeArrayView(FileData), OutInputSize, OutOutputSize, Element);
	}

	void FModelBuilder::WriteFileDataAndReset(TArrayView<uint8> FileData, uint32& OutInputSize, uint32& OutOutputSize, const FModelBuilderElement& Element)
	{
		WriteFileData(FileData, OutInputSize, OutOutputSize, Element);
		Reset();
	}

	void FModelBuilder::WriteFileDataAndReset(TArray<uint8>& FileData, uint32& OutInputSize, uint32& OutOutputSize, const FModelBuilderElement& Element)
	{
		FileData.SetNumUninitialized(GetWriteByteNum(Element));
		WriteFileDataAndReset(MakeArrayView(FileData), OutInputSize, OutOutputSize, Element);
	}

	TArrayView<float> FModelBuilder::MakeValuesCopy(const TConstArrayView<float> Values)
	{
		TArray<float>& OutValues = WeightsPool.AddDefaulted_GetRef();
		OutValues = Values;
		return OutValues;
	}

	TArrayView<float> FModelBuilder::MakeValuesZero(const uint32 Size)
	{
		TArray<float>& Values = WeightsPool.AddDefaulted_GetRef();
		Values.Init(0.0f, Size);
		return Values;
	}

	TArrayView<float> FModelBuilder::MakeValuesOne(const uint32 Size)
	{
		return MakeValuesConstant(Size, 1.0f);
	}

	TArrayView<float> FModelBuilder::MakeValuesConstant(const uint32 Size, const float Value)
	{
		TArray<float>& Values = WeightsPool.AddDefaulted_GetRef();
		Values.Init(Value, Size);
		return Values;
	}

	TArrayView<float> FModelBuilder::MakeWeightsRandomKaimingGaussian(const uint32 InputSize, const uint32 OutputSize, const float Scale)
	{
		TArray<float>& Values = WeightsPool.AddDefaulted_GetRef();
		Values.SetNumUninitialized(InputSize * OutputSize);
		
		const float Std = Scale * FMath::Sqrt(2.0f / InputSize);

		for (uint32 Idx = 0; Idx < InputSize * OutputSize; Idx++)
		{
			Values[Idx] = Std * Private::RngClippedGaussian(Rng);
			Private::RngUpdate(Rng);
		}
		
		return Values;
	}

	TArrayView<float> FModelBuilder::MakeWeightsRandomKaimingUniform(const uint32 InputSize, const uint32 OutputSize, const float Scale)
	{
		TArray<float>& Values = WeightsPool.AddDefaulted_GetRef();
		Values.SetNumUninitialized(InputSize * OutputSize);

		const float Gain = FMath::Sqrt(2.0f / (1.0f + 5.0f));
		const float Std = Gain / FMath::Sqrt((float)InputSize);
		const float Bound = Scale * FMath::Sqrt(3.0f) * Std;

		for (uint32 Idx = 0; Idx < InputSize * OutputSize; Idx++)
		{
			Values[Idx] = Bound * (2.0f * Private::RngUniform(Rng) - 1.0f);
			Private::RngUpdate(Rng);
		}

		return Values;
	}

	TArrayView<float> FModelBuilder::MakeBiasesRandomKaimingGaussian(const uint32 Size, const float Scale)
	{
		TArray<float>& Values = WeightsPool.AddDefaulted_GetRef();
		Values.SetNumUninitialized(Size);

		const float Std = Scale * FMath::Sqrt(2.0f / Size);

		for (uint32 Idx = 0; Idx < Size; Idx++)
		{
			Values[Idx] = Std * Private::RngClippedGaussian(Rng);
			Private::RngUpdate(Rng);
		}

		return Values;
	}

	TArrayView<float> FModelBuilder::MakeBiasesRandomKaimingUniform(const uint32 Size, const float Scale)
	{
		TArray<float>& Values = WeightsPool.AddDefaulted_GetRef();
		Values.SetNumUninitialized(Size);

		const float Bound = Scale / FMath::Sqrt((float)Size);

		for (uint32 Idx = 0; Idx < Size; Idx++)
		{
			Values[Idx] = Bound * (2.0f * Private::RngUniform(Rng) - 1.0f);
			Private::RngUpdate(Rng);
		}

		return Values;
	}

	void FModelBuilder::MakeCompressedWeightsRandomKaimingGaussian(
		TArrayView<uint16>& OutWeightsView,
		TArrayView<float>& OutWeightOffsetsView,
		TArrayView<float>& OutWeightScalesView,
		const uint32 InputSize,
		const uint32 OutputSize,
		const float Scale)
	{
		// Make Kaiming Weights

		TArray<float> Values;
		Values.SetNumUninitialized(InputSize * OutputSize);

		const float Std = Scale * FMath::Sqrt(2.0f / InputSize);

		for (uint32 Idx = 0; Idx < InputSize * OutputSize; Idx++)
		{
			Values[Idx] = Std * Private::RngClippedGaussian(Rng);
			Private::RngUpdate(Rng);
		}

		// Find Min and Max

		TArray<float> Mins;
		TArray<float> Maxs;
		Mins.SetNumUninitialized(InputSize);
		Maxs.SetNumUninitialized(InputSize);

		for (uint32 RowIdx = 0; RowIdx < InputSize; RowIdx++)
		{
			Mins[RowIdx] = +FLT_MAX;
			Maxs[RowIdx] = -FLT_MAX;
		}

		for (uint32 RowIdx = 0; RowIdx < InputSize; RowIdx++)
		{
			for (uint32 ColIdx = 0; ColIdx < OutputSize; ColIdx++)
			{
				Mins[RowIdx] = FMath::Min(Mins[RowIdx], Values[RowIdx * OutputSize + ColIdx]);
				Maxs[RowIdx] = FMath::Max(Maxs[RowIdx], Values[RowIdx * OutputSize + ColIdx]);
			}
		}

		// Find Scale and Offset

		TArray<float>& WeightOffsets = WeightsPool.AddDefaulted_GetRef();
		WeightOffsets.SetNumUninitialized(InputSize);

		TArray<float>& WeightScales = WeightsPool.AddDefaulted_GetRef();
		WeightScales.SetNumUninitialized(InputSize);

		for (uint32 RowIdx = 0; RowIdx < InputSize; RowIdx++)
		{
			WeightOffsets[RowIdx] = Mins[RowIdx];
			WeightScales[RowIdx] = FMath::Max(Maxs[RowIdx] - Mins[RowIdx], UE_SMALL_NUMBER) / 65535.0;
		}

		// Compress

		TArray<uint16>& Weights = CompressedWeightsPool.AddDefaulted_GetRef();
		Weights.SetNumUninitialized(InputSize * OutputSize);

		for (uint32 RowIdx = 0; RowIdx < InputSize; RowIdx++)
		{
			for (uint32 ColIdx = 0; ColIdx < OutputSize; ColIdx++)
			{
				Weights[RowIdx * OutputSize + ColIdx] = (uint16)FMath::RoundToFloat(65535.0 * 
					FMath::Clamp((Values[RowIdx * OutputSize + ColIdx] - Mins[RowIdx]) / (Maxs[RowIdx] - Mins[RowIdx]), 0.0f, 1.0f));
			}
		}

		OutWeightsView = Weights;
		OutWeightOffsetsView = WeightOffsets;
		OutWeightScalesView = WeightScales;
	}

	void FModelBuilder::MakeCompressedWeightsRandomKaimingUniform(
		TArrayView<uint16>& OutWeightsView,
		TArrayView<float>& OutWeightOffsetsView,
		TArrayView<float>& OutWeightScalesView,
		const uint32 InputSize,
		const uint32 OutputSize,
		const float Scale)
	{
		// Make Kaiming Weights

		TArray<float> Values;
		Values.SetNumUninitialized(InputSize * OutputSize);

		const float Gain = FMath::Sqrt(2.0f / (1.0f + 5.0f));
		const float Std = Gain / FMath::Sqrt((float)InputSize);
		const float Bound = Scale * FMath::Sqrt(3.0f) * Std;

		for (uint32 Idx = 0; Idx < InputSize * OutputSize; Idx++)
		{
			Values[Idx] = Bound * (2.0f * Private::RngUniform(Rng) - 1.0f);
			Private::RngUpdate(Rng);
		}

		// Find Min and Max

		TArray<float> Mins;
		TArray<float> Maxs;
		Mins.SetNumUninitialized(InputSize);
		Maxs.SetNumUninitialized(InputSize);

		for (uint32 RowIdx = 0; RowIdx < InputSize; RowIdx++)
		{
			Mins[RowIdx] = +FLT_MAX;
			Maxs[RowIdx] = -FLT_MAX;
		}

		for (uint32 RowIdx = 0; RowIdx < InputSize; RowIdx++)
		{
			for (uint32 ColIdx = 0; ColIdx < OutputSize; ColIdx++)
			{
				Mins[RowIdx] = FMath::Min(Mins[RowIdx], Values[RowIdx * OutputSize + ColIdx]);
				Maxs[RowIdx] = FMath::Max(Maxs[RowIdx], Values[RowIdx * OutputSize + ColIdx]);
			}
		}

		// Find Scale and Offset

		TArray<float>& WeightOffsets = WeightsPool.AddDefaulted_GetRef();
		WeightOffsets.SetNumUninitialized(InputSize);

		TArray<float>& WeightScales = WeightsPool.AddDefaulted_GetRef();
		WeightScales.SetNumUninitialized(InputSize);

		for (uint32 RowIdx = 0; RowIdx < InputSize; RowIdx++)
		{
			WeightOffsets[RowIdx] = Mins[RowIdx];
			WeightScales[RowIdx] = FMath::Max(Maxs[RowIdx] - Mins[RowIdx], UE_SMALL_NUMBER) / 65535.0f;
		}

		// Compress

		TArray<uint16>& Weights = CompressedWeightsPool.AddDefaulted_GetRef();
		Weights.SetNumUninitialized(InputSize * OutputSize);

		for (uint32 RowIdx = 0; RowIdx < InputSize; RowIdx++)
		{
			for (uint32 ColIdx = 0; ColIdx < OutputSize; ColIdx++)
			{
				Weights[RowIdx * OutputSize + ColIdx] = (uint16)FMath::RoundToFloat(65535.0f *
					FMath::Clamp((Values[RowIdx * OutputSize + ColIdx] - Mins[RowIdx]) / FMath::Max(Maxs[RowIdx] - Mins[RowIdx], UE_SMALL_NUMBER), 0.0f, 1.0f));
			}
		}

		OutWeightsView = Weights;
		OutWeightOffsetsView = WeightOffsets;
		OutWeightScalesView = WeightScales;
	}

	TArrayView<float> FModelBuilder::MakeInitialWeights(
		const uint32 InputSize,
		const uint32 OutputSize,
		const FWeightInitializationSettings& WeightInitializationSettings)
	{
		switch (WeightInitializationSettings.Type)
		{
		case EWeightInitializationType::KaimingGaussian:

			return MakeWeightsRandomKaimingGaussian(InputSize, OutputSize, WeightInitializationSettings.Scale);

		case EWeightInitializationType::KaimingUniform:

			return MakeWeightsRandomKaimingUniform(InputSize, OutputSize, WeightInitializationSettings.Scale);

		default:
			checkNoEntry();
			return TArrayView<float>();
		}
	}

	TArrayView<float> FModelBuilder::MakeInitialBiases(
		const uint32 OutputSize,
		const FWeightInitializationSettings& WeightInitializationSettings)
	{
		if (!WeightInitializationSettings.bInitializeBiases)
		{
			return MakeValuesZero(OutputSize);
		}

		switch (WeightInitializationSettings.Type)
		{
		case EWeightInitializationType::KaimingGaussian:

			return MakeBiasesRandomKaimingGaussian(OutputSize, WeightInitializationSettings.Scale);

		case EWeightInitializationType::KaimingUniform:

			return MakeBiasesRandomKaimingUniform(OutputSize, WeightInitializationSettings.Scale);

		default:
			checkNoEntry();
			return TArrayView<float>();
		}
	}

	void FModelBuilder::MakeInitialCompressedWeights(
		TArrayView<uint16>& OutWeightsView,
		TArrayView<float>& OutWeightOffsetsView,
		TArrayView<float>& OutWeightScalesView,
		const uint32 InputSize,
		const uint32 OutputSize,
		const FWeightInitializationSettings& WeightInitializationSettings)
	{
		switch (WeightInitializationSettings.Type)
		{
		case EWeightInitializationType::KaimingGaussian:

			MakeCompressedWeightsRandomKaimingGaussian(
				OutWeightsView, 
				OutWeightOffsetsView, 
				OutWeightScalesView, 
				InputSize, 
				OutputSize, 
				WeightInitializationSettings.Scale);

			return;

		case EWeightInitializationType::KaimingUniform:

			MakeCompressedWeightsRandomKaimingUniform(
				OutWeightsView,
				OutWeightOffsetsView,
				OutWeightScalesView,
				InputSize,
				OutputSize,
				WeightInitializationSettings.Scale);

			return;

		default:
			checkNoEntry();
			return;
		}
	}

	TArrayView<uint32> FModelBuilder::MakeSizesZero(const uint32 Size)
	{
		TArray<uint32>& Values = SizesPool.AddDefaulted_GetRef();
		Values.Init(0, Size);
		return Values;
	}

	TArrayView<uint32> FModelBuilder::MakeSizesLayerInputs(const TConstArrayView<FModelBuilderElement> Elements)
	{
		const uint32 SizeNum = Elements.Num();
		TArrayView<uint32> Sizes = MakeSizesZero(SizeNum);
		for (uint32 SizeIdx = 0; SizeIdx < SizeNum; SizeIdx++)
		{
			Sizes[SizeIdx] = Elements[SizeIdx].GetInputSize();
		}

		return Sizes;
	}

	TArrayView<uint32> FModelBuilder::MakeSizesLayerOutputs(const TConstArrayView<FModelBuilderElement> Elements)
	{
		const uint32 SizeNum = Elements.Num();
		TArrayView<uint32> Sizes = MakeSizesZero(SizeNum);
		for (uint32 SizeIdx = 0; SizeIdx < SizeNum; SizeIdx++)
		{
			Sizes[SizeIdx] = Elements[SizeIdx].GetOutputSize();
		}

		return Sizes;
	}

	//--------------------------------------------------------------------------

} // namespace UE::NNE::RuntimeBasic

#undef NNE_RUNTIME_BASIC_CHECK_ALIASING
#undef NNE_RUNTIME_BASIC_TRACE_SCOPE
#undef NNE_RUNTIME_BASIC_ENABLE_ALIASING_CHECK
#undef NNE_RUNTIME_BASIC_ENABLE_PROFILE
#undef NNE_RUNTIME_BASIC_ENABLE_NAN_CHECK
#undef NNE_RUNTIME_BASIC_ENABLE_ISPC