// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNERuntimeCPU.h"
#include "NNETypes.h"

namespace UE::NNE
{
	class FSharedModelData;
}

namespace UE::NNE::RuntimeBasic
{
	namespace Private
	{
		struct ILayer;
		struct ILayerInstance;
	}

	class FModelCPU;
	class FModelInstanceCPU;

	/**
	 * Basic implementation of a model instance used for performing inference in this runtime.
	 */
	class FModelInstanceCPU : public IModelInstanceCPU
	{
	public:
		using ESetInputTensorShapesStatus = IModelInstanceCPU::ESetInputTensorShapesStatus;
		using ERunSyncStatus = IModelInstanceCPU::ERunSyncStatus;

		FModelInstanceCPU(const TSharedPtr<FModelCPU>& InModel);

		//~ Begin UE::NNE::IModelInstanceCPU Interface
		virtual TConstArrayView<FTensorDesc> GetInputTensorDescs() const override final { return TConstArrayView<FTensorDesc>(&InputTensorDesc, 1); }
		virtual TConstArrayView<FTensorDesc> GetOutputTensorDescs() const override final { return TConstArrayView<FTensorDesc>(&OutputTensorDesc, 1); }
		virtual TConstArrayView<FTensorShape> GetInputTensorShapes() const override final { return TConstArrayView<FTensorShape>(&InputTensorShape, 1); }
		virtual TConstArrayView<FTensorShape> GetOutputTensorShapes() const override final { return TConstArrayView<FTensorShape>(&OutputTensorShape, 1); }
		virtual ESetInputTensorShapesStatus SetInputTensorShapes(TConstArrayView<FTensorShape> InInputShapes) override final;
		virtual ERunSyncStatus RunSync(TConstArrayView<FTensorBindingCPU> InInputBindings, TConstArrayView<FTensorBindingCPU> InOutputBindings) override final;
		//~ End UE::NNE::IModelInstanceCPU Interface

		TSharedPtr<FModelCPU> Model;
		FTensorDesc InputTensorDesc;
		FTensorDesc OutputTensorDesc;
		FTensorShape InputTensorShape;
		FTensorShape OutputTensorShape;
		TSharedPtr<Private::ILayerInstance> Instance;

		uint32 BatchSize = 0;
		uint32 InputSize = 0;
		uint32 OutputSize = 0;
	};

	/**
	 * Basic implementation of a model storing the network weights in this runtime.
	 */
	class FModelCPU : public IModelCPU
	{
	public:

		/** Magic number for the serialization format of this model data */
		static uint32 ModelMagicNumber;

		/** Version number for the serialization format of this model data */
		static uint32 ModelVersionNumber;

		//~ Begin UE::NNE::IModelCPU Interface
		virtual TSharedPtr<IModelInstanceCPU> CreateModelInstanceCPU() override final;
		//~ End UE::NNE::IModelCPU Interface

		/** Computes the number of bytes required to save or load this model */
		void SerializationSize(uint64& InOutOffset) const;

		/** Loads the model from the array of bytes starting at the given offset */
		bool SerializationLoad(uint64& InOutOffset, TConstArrayView<uint8> Data);

		/** Saves the model to the array of bytes starting at the given offset */
		void SerializationSave(uint64& InOutOffset, TArrayView<uint8> Data) const;

		/** Private implementation of the model */
		TSharedPtr<Private::ILayer> Layer;

		/** Shared pointer to the actual model data */
		TSharedPtr<FSharedModelData> ModelData;

		/**
		 * We need to store a TWeakPtr to ourselves so that when we construct the instance we can pass it a TSharedPtr 
		 * to this object, which ensures the Model does not get deleted while there are still instances alive.
		 */
		TWeakPtr<FModelCPU> WeakThis;
	};
}