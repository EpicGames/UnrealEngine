// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeBasicCpuModule.h"
#include "NNE.h"
#include "NNERuntimeBasicCpu.h"
#include "UObject/WeakInterfacePtr.h"

void FNNERuntimeBasicCpuModule::StartupModule()
{
	// NNE runtime startup
	NNERuntimeBasicCpu = NewObject<UNNERuntimeBasicCpuImpl>();
	if (NNERuntimeBasicCpu.IsValid())
	{
		TWeakInterfacePtr<INNERuntime> RuntimeCPUInterface(NNERuntimeBasicCpu.Get());

		NNERuntimeBasicCpu->AddToRoot();
		UE::NNE::RegisterRuntime(RuntimeCPUInterface);
	}
}

void FNNERuntimeBasicCpuModule::ShutdownModule()
{
	// NNE runtime shutdown
	if (NNERuntimeBasicCpu.IsValid())
	{
		TWeakInterfacePtr<INNERuntime> RuntimeCPUInterface(NNERuntimeBasicCpu.Get());

		UE::NNE::UnregisterRuntime(RuntimeCPUInterface);
		NNERuntimeBasicCpu.Reset();
	}
}

IMPLEMENT_MODULE(FNNERuntimeBasicCpuModule, NNERuntimeBasicCpu);
