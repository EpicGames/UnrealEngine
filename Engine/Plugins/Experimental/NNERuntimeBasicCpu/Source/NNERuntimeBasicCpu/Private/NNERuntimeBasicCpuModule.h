// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UNNERuntimeBasicCpuImpl;

class FNNERuntimeBasicCpuModule : public IModuleInterface
{
public:
	TWeakObjectPtr<UNNERuntimeBasicCpuImpl> NNERuntimeBasicCpu{ nullptr };

	// Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};