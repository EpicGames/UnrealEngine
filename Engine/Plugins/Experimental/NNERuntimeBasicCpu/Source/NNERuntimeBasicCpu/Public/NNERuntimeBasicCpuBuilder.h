// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <Math/RandomStream.h>

#define UE_API NNERUNTIMEBASICCPU_API

namespace UE::NNE::RuntimeBasic
{
	namespace Private
	{
		struct ILayer;
	}

	class FModelBuilder;

	/**
	 * Represents an element of a model (such as a layer) created by the Model Builder.
	 */
	class FModelBuilderElement
	{
		friend class FModelBuilder;

	public:

		UE_API FModelBuilderElement();
		UE_API FModelBuilderElement(const TSharedPtr<Private::ILayer>& Ptr);
		UE_API ~FModelBuilderElement();

		UE_API int32 GetInputSize() const;
		UE_API int32 GetOutputSize() const;

	private:
		TSharedPtr<Private::ILayer> Layer;
	};

	/**
	 * This class can be used to construct Models for use with the Basic CPU Runtime. 
	 * Effectively it works by constructing the model in-memory and then serializing this
	 * out to create a FileData object which can then be loaded with NNE.
	 * 
	 * Note: If you pass your own views into this model builder they must out-live the
	 * builder itself as it will not create internal copies. If you want to create an
	 * internal copy use the `MakeValuesCopy` function.
	 */
	class FModelBuilder
	{

	public:

		/** Common activation function types. */
		enum class EActivationFunction : uint8
		{
			ReLU = 0,
			ELU = 1,
			TanH = 2,
			GELU = 3,
		};

		/** Linear Layer types. */
		enum class ELinearLayerType : uint8
		{
			Normal = 0,
			Compressed = 1,
			Lipschizt = 2,
		};

		/** Weight initialization types. */
		enum class EWeightInitializationType : uint8
		{
			// Good all-round default weight initialization
			KaimingGaussian = 0,

			// Default initialization used in PyTorch for Linear Layers
			KaimingUniform = 1,
		};

		/** Padding Mode for Convolutions */
		enum class EPaddingMode : uint8
		{
			// The input will be padded with zeroes on both sides
			Zeros = 0, 
			// The input will be padded with wrap around values
			Circular = 1,
		};

		/*
		 * There is a clang compiler bug that requires the following classes to declare a default constructor in the cpp file as a workaround:
		 * 
		 * https://github.com/llvm/llvm-project/issues/36032
		 * https://stackoverflow.com/a/73248547
		 */

		/** Weight initialization settings. */
		struct FWeightInitializationSettings
		{
			// Workaround for clang compiler bug...
			UE_API FWeightInitializationSettings();

			// Type of initialization to use
			EWeightInitializationType Type = EWeightInitializationType::KaimingUniform;

			// Overall scale of the weight initialization
			float Scale = 1.0f;

			// If true, initializes biases using the given initialization method, otherwise initializes them to zeros
			bool bInitializeBiases = true;
		};

		/** Linear Layer settings including the type of layer and the way the weights are initialized */
		struct FLinearLayerSettings
		{
			// Workaround for clang compiler bug...
			UE_API FLinearLayerSettings();

			// Type of the Linear Layer to construct
			ELinearLayerType Type = ELinearLayerType::Normal;

			// Weight initialization Settings for the linear Layer
			FWeightInitializationSettings WeightInitializationSettings;
		};

		/** Construct a new Model Builder with the given random seed. */
		UE_API FModelBuilder(int32 Seed = 0x0a974e75);

		/**
		 * Makes a new linear layer.
		 *
		 * @param InputSize		Input Vector Size (Number of Rows)
		 * @param OutputSize	Output Vector Size (Number of Columns)
		 * @param Weights		Linear layer weights.
		 * @param Biases		Linear layer biases.
		 */
		UE_API FModelBuilderElement MakeLinear(
			const uint32 InputSize,
			const uint32 OutputSize,
			const TConstArrayView<float> Weights,
			const TConstArrayView<float> Biases);

		/**
		 * Makes a new compressed linear layer.
		 *
		 * @param InputSize		Input Vector Size (Number of Rows)
		 * @param OutputSize	Output Vector Size (Number of Columns)
		 * @param Weights		Compressed Linear layer weights.
		 * @param WeightOffsets	Compressed Linear layer weight offsets.
		 * @param WeightScales	Compressed Linear layer weight biases.
		 * @param Biases		Compressed Linear layer biases.
		 */
		UE_API FModelBuilderElement MakeCompressedLinear(
			const uint32 InputSize,
			const uint32 OutputSize,
			const TConstArrayView<uint16> Weights,
			const TConstArrayView<float> WeightOffsets,
			const TConstArrayView<float> WeightScales,
			const TConstArrayView<float> Biases);

		/**
		 * Makes a new Lipschizt linear layer.
		 *
		 * @param InputSize		Input Vector Size (Number of Rows)
		 * @param OutputSize	Output Vector Size (Number of Columns)
		 * @param Weights		Linear layer weights.
		 * @param Biases		Linear layer biases.
		 */
		UE_API FModelBuilderElement MakeLipschiztLinear(
			const uint32 InputSize,
			const uint32 OutputSize,
			const TConstArrayView<float> Weights,
			const TConstArrayView<float> Biases);

		/**
		 * Makes a new linear layer with the given settings.
		 *
		 * @param InputSize		Input Vector Size (Number of Rows)
		 * @param OutputSize	Output Vector Size (Number of Columns)
		 * @param Settings		Layer Settings.
		 */
		UE_API FModelBuilderElement MakeLinearLayer(
			const uint32 InputSize,
			const uint32 OutputSize,
			const FLinearLayerSettings& Settings = FLinearLayerSettings());

		/**
		 * Makes a new multi linear layer.
		 *
		 * @param InputSize		Input Vector Size (Number of Rows)
		 * @param OutputSize	Output Vector Size (Number of Columns)
		 * @param BlockNum		Number of blocks (Number of Matrices)
		 * @param Weights		Multi-Linear layer weights.
		 * @param Biases		Multi-Linear layer biases.
		 */
		UE_API FModelBuilderElement MakeMultiLinear(
			const uint32 InputSize,
			const uint32 OutputSize,
			const uint32 BlockNum,
			const TConstArrayView<float> Weights,
			const TConstArrayView<float> Biases);

		/**
		 * Makes a new Normalization layer.
		 * 
		 * @param Mean		Normalization Mean.
		 * @param Std		Normalization Std.
		 */
		UE_API FModelBuilderElement MakeNormalize(
			const uint32 InputOutputSize,
			const TConstArrayView<float> Mean,
			const TConstArrayView<float> Std);

		/**
		 * Makes a new Denormalization layer.
		 *
		 * @param Mean		Denormalization Mean.
		 * @param Std		Denormalization Std.
		 */
		UE_API FModelBuilderElement MakeDenormalize(
			const uint32 InputOutputSize,
			const TConstArrayView<float> Mean,
			const TConstArrayView<float> Std);

		/** Makes a ReLU Activation Layer */
		UE_API FModelBuilderElement MakeReLU(const uint32 InputOutputSize);

		/** Makes a ELU Activation Layer */
		UE_API FModelBuilderElement MakeELU(const uint32 InputOutputSize);

		/** Makes a GELU Activation Layer */
		UE_API FModelBuilderElement MakeGELU(const uint32 InputOutputSize);

		/** Makes a TanH Activation Layer */
		UE_API FModelBuilderElement MakeTanH(const uint32 InputOutputSize);

		/** Makes a Softmax Layer */
		UE_API FModelBuilderElement MakeSoftmax(const uint32 InputOutputSize);

		/** Makes a Copy Layer */
		UE_API FModelBuilderElement MakeCopy(const uint32 InputOutputSize);

		/** Makes a Slice Layer */
		UE_API FModelBuilderElement MakeSlice(const uint32 InputSize, const uint32 SliceOffset, const uint32 SliceSize);

		/** Makes a Clamp Layer */
		UE_API FModelBuilderElement MakeClamp(const uint32 InputOutputSize, const TConstArrayView<float> MinValues, const TConstArrayView<float> MaxValues);

		/** Makes a new activation layer with the given activation function */
		UE_API FModelBuilderElement MakeActivation(const uint32 InputOutputSize, const EActivationFunction ActivationFunction);

		/**
		 * Makes a PReLU Activation Layer
		 *
		 * @param LayerSize			Number of neurons in layer.
		 * @param Alpha				PReLU alpha parameter for each neuron.
		 */
		UE_API FModelBuilderElement MakePReLU(const uint32 InputOutputSize, const TConstArrayView<float> Alpha);

		/**
		 * Makes a Sequence layer, which will evaluate the given list of layers in order.
		 */
		UE_API FModelBuilderElement MakeSequence(const TConstArrayView<FModelBuilderElement> Elements);

		/**
		 * Makes a Multi-Layer Perceptron network.
		 *
		 * @param InputSize					Input Vector Size
		 * @param OutputSize				Output Vector Size
		 * @param HiddenSize				Number of hidden units to use on internal layers.
		 * @param LayerNum					Number of layers. Includes input and output layers.
		 * @param ActivationFunction		Activation function to use.
		 * @param bActivationOnFinalLayer	If the activation function should be used on the final output layer.
		 * @param LinearLayerSettings		Settings for the linear layers in the MLP.
		 */
		UE_API FModelBuilderElement MakeMLP(
			const uint32 InputSize,
			const uint32 OutputSize, 
			const uint32 HiddenSize,
			const uint32 LayerNum,
			const EActivationFunction ActivationFunction,
			const bool bActivationOnFinalLayer = false,
			const FLinearLayerSettings& LinearLayerSettings = FLinearLayerSettings());

		/**
		 * Makes a Multi-Layer Perceptron network with LayerNorm before all activations.
		 *
		 * @param InputSize					Input Vector Size
		 * @param OutputSize				Output Vector Size
		 * @param HiddenSize				Number of hidden units to use on internal layers.
		 * @param LayerNum					Number of layers. Includes input and output layers.
		 * @param ActivationFunction		Activation function to use.
		 * @param bActivationOnFinalLayer	If the activation function should be used on the final output layer.
		 * @param LinearLayerSettings		Settings for the linear layers in the MLP.
		 */
		UE_API FModelBuilderElement MakeMLPWithLayerNorm(
			const uint32 InputSize,
			const uint32 OutputSize,
			const uint32 HiddenSize,
			const uint32 LayerNum,
			const EActivationFunction ActivationFunction,
			const bool bActivationOnFinalLayer = false,
			const FLinearLayerSettings& LinearLayerSettings = FLinearLayerSettings());

		/**
		 * Makes a Multi-Layer Perceptron network using Skip layers that concatenate the input to each intermediate layer.
		 *
		 * @param InputSize					Input Vector Size
		 * @param OutputSize				Output Vector Size
		 * @param HiddenSize				Number of hidden units to use on internal layers.
		 * @param LayerNum					Number of layers. Includes input and output layers.
		 * @param ActivationFunction		Activation function to use.
		 * @param bActivationOnFinalLayer	If the activation function should be used on the final output layer.
		 * @param LinearLayerSettings		Settings for the linear layers in the MLP.
		 */
		UE_API FModelBuilderElement MakeSkipMLP(
			const uint32 InputSize,
			const uint32 OutputSize,
			const uint32 HiddenSize,
			const uint32 LayerNum,
			const EActivationFunction ActivationFunction,
			const bool bActivationOnFinalLayer = false,
			const FLinearLayerSettings& LinearLayerSettings = FLinearLayerSettings());

		/**
		 * Makes a Multi-Layer Perceptron network using Skip layers that concatenate the input to each intermediate layer followed by a LayerNorm.
		 *
		 * @param InputSize					Input Vector Size
		 * @param OutputSize				Output Vector Size
		 * @param HiddenSize				Number of hidden units to use on internal layers.
		 * @param LayerNum					Number of layers. Includes input and output layers.
		 * @param ActivationFunction		Activation function to use.
		 * @param bActivationOnFinalLayer	If the activation function should be used on the final output layer.
		 * @param LinearLayerSettings		Settings for the linear layers in the MLP.
		 */
		UE_API FModelBuilderElement MakeSkipMLPWithLayerNorm(
			const uint32 InputSize,
			const uint32 OutputSize,
			const uint32 HiddenSize,
			const uint32 LayerNum,
			const EActivationFunction ActivationFunction,
			const bool bActivationOnFinalLayer = false,
			const FLinearLayerSettings& LinearLayerSettings = FLinearLayerSettings());
		
		/**
		 * Makes a Multi-Layer Perceptron network using Residual layers.
		 *
		 * @param InputSize					Input Vector Size
		 * @param OutputSize				Output Vector Size
		 * @param HiddenSize				Number of hidden units to use on internal layers.
		 * @param LayerNum					Number of layers. Includes input and output layers.
		 * @param ActivationFunction		Activation function to use.
		 * @param bActivationOnFinalLayer	If the activation function should be used on the final output layer.
		 * @param LinearLayerSettings		Settings for the linear layers in the MLP.
		 */
		UE_API FModelBuilderElement MakeResidualMLP(
			const uint32 InputSize,
			const uint32 OutputSize,
			const uint32 HiddenSize,
			const uint32 LayerNum,
			const EActivationFunction ActivationFunction,
			const bool bActivationOnFinalLayer = false,
			const FLinearLayerSettings& LinearLayerSettings = FLinearLayerSettings());

		/**
		 * Makes a Multi-Layer Perceptron network using Residual layers followed by LayerNorm layers.
		 *
		 * @param InputSize					Input Vector Size
		 * @param OutputSize				Output Vector Size
		 * @param HiddenSize				Number of hidden units to use on internal layers.
		 * @param LayerNum					Number of layers. Includes input and output layers.
		 * @param ActivationFunction		Activation function to use.
		 * @param bActivationOnFinalLayer	If the activation function should be used on the final output layer.
		 * @param LinearLayerSettings		Settings for the linear layers in the MLP.
		 */
		UE_API FModelBuilderElement MakeResidualMLPWithLayerNorm(
			const uint32 InputSize,
			const uint32 OutputSize,
			const uint32 HiddenSize,
			const uint32 LayerNum,
			const EActivationFunction ActivationFunction,
			const bool bActivationOnFinalLayer = false,
			const FLinearLayerSettings& LinearLayerSettings = FLinearLayerSettings());

		/**
		 * Make a new Memory Cell layer.
		 *
		 * @param InputNum					Number of normal inputs to the model.
		 * @param OutputNum					Number of normal outputs from the model.
		 * @param MemoryNum					The size of the memory vector used by the model.
		 * @param RememberLayer				Layer used for the Remember Gate.
		 * @param PassthroughLayer			Layer used for the Passthrough Gate.
		 * @param MemoryUpdateLayer			Layer used for updating the memory Gate.
		 * @param OutputInputUpdateLayer	Layer used for updating the output from the input.
		 * @param OutputMemoryUpdateLayer	Layer used for updating the output from the memory.
		 */
		UE_API FModelBuilderElement MakeMemoryCell(
			const uint32 InputNum,
			const uint32 OutputNum,
			const uint32 MemoryNum,
			const FModelBuilderElement& RememberLayer,
			const FModelBuilderElement& PassthroughLayer,
			const FModelBuilderElement& MemoryUpdateLayer,
			const FModelBuilderElement& OutputInputUpdateLayer,
			const FModelBuilderElement& OutputMemoryUpdateLayer);

		/**
		 * Make a new Memory Cell layer with the given linear layer settings.
		 *
		 * @param InputNum				Number of normal inputs to the model
		 * @param OutputNum				Number of normal outputs from the model
		 * @param MemoryNum				The size of the memory vector used by the model
		 * @param LinearLayerSettings	Settings for the linear layers in the Memory Cell.
		 */
		UE_API FModelBuilderElement MakeMemoryCellLayer(
			const uint32 InputNum,
			const uint32 OutputNum,
			const uint32 MemoryNum,
			const FLinearLayerSettings& LinearLayerSettings = FLinearLayerSettings());

		/**
		 * Make a new GRU Cell layer.
		 *
		 * @param InputNum					Number of inputs to the model.
		 * @param OutputNum					Number of outputs from the model. Will also be the memory size.
		 * @param RememberLayer				Layer used for the Remember Gate.
		 * @param UpdateLayer				Layer used for the Update Gate.
		 * @param ActivationLayer			Layer used for producing the Activation values.
		 */
		UE_API FModelBuilderElement MakeGRUCell(
			const uint32 InputNum,
			const uint32 OutputNum,
			const FModelBuilderElement& RememberLayer,
			const FModelBuilderElement& UpdateLayer,
			const FModelBuilderElement& ActivationLayer);

		/**
		 * Make a new GRU Cell layer with the given linear layer settings.
		 *
		 * @param InputNum				Number of inputs to the model.
		 * @param OutputNum				Number of outputs from the model. Will also be the memory size.
		 * @param LinearLayerSettings	Settings for the linear layers in the Memory Cell.
		 */
		UE_API FModelBuilderElement MakeGRUCellLayer(
			const uint32 InputNum,
			const uint32 OutputNum,
			const FLinearLayerSettings& LinearLayerSettings = FLinearLayerSettings());

		/**
		 * Make a new Memory Backbone layer.
		 *
		 * @param MemoryNum				The size of the memory vector used by the model
		 * @param Prefix				Prefix Element
		 * @param Cell					Memory Cell Element
		 * @param Postfix				Postfix Element
		 */
		UE_API FModelBuilderElement MakeMemoryBackbone(
			const uint32 MemoryNum,
			const FModelBuilderElement& Prefix,
			const FModelBuilderElement& Cell,
			const FModelBuilderElement& Postfix);

		/**
		 * Makes a Concat layer, which will evaluate each of the given elements on different slices of 
		 * the input vector, concatenating the result into the output vector.
		 */
		UE_API FModelBuilderElement MakeConcat(const TConstArrayView<FModelBuilderElement> Elements);
		
		/**
		 * Makes a Spread layer, which will evaluate each of the given elements on the input vector, 
		 * concatenating the result into the output vector.
		 */
		UE_API FModelBuilderElement MakeSpread(const TConstArrayView<FModelBuilderElement> Elements);

		/**
		 * Make a layer which runs the given sublayer on an array of elements.
		 * 
		 * @param ElementNum	Number of elements in the array.
		 * @param SubLayer		Sublayer to evaluate on each item in the array.
		 */
		UE_API FModelBuilderElement MakeArray(const uint32 ElementNum, const FModelBuilderElement& SubLayer);

		/**
		 * Make a residual layer which adds the result of evaluating the given sublayer on the input, to the input.
		 *
		 * @param SubLayer		Sublayer to evaluate on the input.
		 */
		UE_API FModelBuilderElement MakeResidual(const FModelBuilderElement& SubLayer);

		/**
		 * Make a layer which aggregates a set of other observations using attention. This is used by LearningAgents.
		 */
		UE_API FModelBuilderElement MakeAggregateSet(
			const uint32 MaxElementNum, 
			const uint32 OutputEncodingSize,
			const uint32 AttentionEncodingSize,
			const uint32 AttentionHeadNum,
			const FModelBuilderElement& SubLayer,
			const FModelBuilderElement& QueryLayer,
			const FModelBuilderElement& KeyLayer,
			const FModelBuilderElement& ValueLayer);

		/**
		 * Make a layer which aggregates an Exclusive Or of other observations. This is used by LearningAgents.
		 */
		UE_API FModelBuilderElement MakeAggregateOrExclusive(
			const uint32 OutputEncodingSize,
			const TConstArrayView<FModelBuilderElement> SubLayers,
			const TConstArrayView<FModelBuilderElement> Encoders);

		/**
		 * Make a layer which aggregates an Inclusive Or of other observations using attention. This is used by LearningAgents.
		 */
		UE_API FModelBuilderElement MakeAggregateOrInclusive(
			const uint32 OutputEncodingSize,
			const uint32 AttentionEncodingSize,
			const uint32 AttentionHeadNum,
			const TConstArrayView<FModelBuilderElement> SubLayers,
			const TConstArrayView<FModelBuilderElement> QueryLayers,
			const TConstArrayView<FModelBuilderElement> KeyLayers,
			const TConstArrayView<FModelBuilderElement> ValueLayers);

		/**
		 * Make a new Top-Two Sparse Mixture of Experts Layer.
		 *
		 * @param InputNum					Number of normal inputs to the model.
		 * @param OutputNum					Number of normal outputs from the model.
		 * @param GatingLayer				Layer used to choose the experts. Input should be of size InputNum, output should be of size SubLayerNum.
		 * @param SubLayers					Expert layers. All Layers here should have input of size InputNum, output of size OutputNum.
		 */
		UE_API FModelBuilderElement MakeSparseMixtureOfExperts(
			const uint32 InputNum,
			const uint32 OutputNum,
			const FModelBuilderElement& GatingLayer,
			const TConstArrayView<FModelBuilderElement> SubLayers);

		/**
		 * Makes a new Layer Norm layer.
		 *
		 * @param InputOutputSize	Input and Output Vector Size
		 * @param Offsets			Initial Offsets
		 * @param Scales			Initial Scales
		 * @param Epsilon			Standard Deviation Epsilon
		 */
		UE_API FModelBuilderElement MakeLayerNorm(
			const uint32 InputOutputSize,
			const TConstArrayView<float> Offsets,
			const TConstArrayView<float> Scales,
			const float Epsilon = 1e-5f);

		/** Makes a Tile Layer */
		UE_API FModelBuilderElement MakeTile(const uint32 InputSize, const uint32 Repeats);

		/**
		 * Make a new FiLM conditioned Network.
		 *
		 * @param Prefix				Prefix Element
		 * @param Condition				Condition Element
		 * @param Postfix				Postfix Element
		 */
		UE_API FModelBuilderElement MakeFiLMNetwork(
			const FModelBuilderElement& Prefix,
			const FModelBuilderElement& Condition,
			const FModelBuilderElement& Postfix);
		
		/**
		 * Make a new 1D Convolution Layer.
		 *
		 * @param InputLength			Length of 1 dimensional input
		 * @param InChannels			Number of channels in the input
		 * @param OutChannels			Number of channels produced by the convolution
		 * @param KernelSize			Size of the convolving kernel
		 * @param Padding				The amount of padding added to both sides of the input
		 * @param PaddingMode			"Zeros" or "Circular"
		 * @param Weights				Convolutional layer weights
		 * @param Biases				Convolutional layer biases
		 * @param SubLayer				The sublayer to evaluate on the input
		 */
		UE_API FModelBuilderElement MakeConv1d(
			const uint32 InputLength,
			const uint32 InChannels,
			const uint32 OutChannels,
			const uint32 KernelSize,
			const uint32 Padding, 
			const EPaddingMode PaddingMode,
			const TConstArrayView<float> Weights,
			const TConstArrayView<float> Biases,
			const FModelBuilderElement& SubLayer);

		/**
		* Make a new 2D Convolution Layer.
		*
		* @param InputHeight			Height of the 2d flat buffer input
		* @param InputWidth				Width of the 2d flat buffer input
		* @param InChannels				Number of channels in the input
		* @param OutChannels			Number of channels produced by the convolution
		* @param KernelSize				Size of the convolving kernel
		* @param Stride					The stride of the cross correlation 
		* @param Padding				The amount of padding added to both sides of the input
		* @param PaddingMode			"Zeros" or "Circular"
		* @param Weights				Convolutional layer weights
		* @param Biases					Convolutional layer biases
		* @param SubLayer				The sublayer to evaluate on the input
		*/
		UE_API FModelBuilderElement MakeConv2d(
			const uint32 InputHeight,
			const uint32 InputWidth,
			const uint32 InChannels,
			const uint32 OutChannels,
			const uint32 KernelSize,
			const uint32 Stride,
			const uint32 Padding,
			const EPaddingMode PaddingMode,
			const TConstArrayView<float> Weights,
			const TConstArrayView<float> Biases,
			const FModelBuilderElement& SubLayer);

	public:

		/** Creates a array of values from a copy of the given array view */
		UE_API TArrayView<float> MakeValuesCopy(const TConstArrayView<float> Values);

		/** Creates a array of values set to zero of the given size */
		UE_API TArrayView<float> MakeValuesZero(const uint32 Size);

		/** Creates a array of values set to one of the given size */
		UE_API TArrayView<float> MakeValuesOne(const uint32 Size);

		/** Creates a array of values set to the provided constant value of the given size */
		UE_API TArrayView<float> MakeValuesConstant(const uint32 Size, const float Value);

		/** Creates a array of weights randomly initialized using the Gaussian Kaiming method */
		UE_API TArrayView<float> MakeWeightsRandomKaimingGaussian(const uint32 InputSize, const uint32 OutputSize, const float Scale = 1.0f);

		/** Creates a array of weights randomly initialized using the Uniform Kaiming method */
		UE_API TArrayView<float> MakeWeightsRandomKaimingUniform(const uint32 InputSize, const uint32 OutputSize, const float Scale = 1.0f);

		/** Creates a array of biases randomly initialized using the Gaussian Kaiming method */
		UE_API TArrayView<float> MakeBiasesRandomKaimingGaussian(const uint32 Size, const float Scale = 1.0f);

		/** Creates a array of biases randomly initialized using the Uniform Kaiming method# */
		UE_API TArrayView<float> MakeBiasesRandomKaimingUniform(const uint32 Size, const float Scale = 1.0f);

		/** Creates a array of weights randomly initialized using the Gaussian Kaiming method and compresses them */
		UE_API void MakeCompressedWeightsRandomKaimingGaussian(
			TArrayView<uint16>& OutWeightsView,
			TArrayView<float>& OutWeightOffsetsView,
			TArrayView<float>& OutWeightScalesView,
			const uint32 InputSize,
			const uint32 OutputSize,
			const float Scale = 1.0f);

		/** Creates a array of weights randomly initialized using the Uniform Kaiming method and compresses them */
		UE_API void MakeCompressedWeightsRandomKaimingUniform(
			TArrayView<uint16>& OutWeightsView,
			TArrayView<float>& OutWeightOffsetsView,
			TArrayView<float>& OutWeightScalesView,
			const uint32 InputSize,
			const uint32 OutputSize,
			const float Scale = 1.0f);

		/** Creates a array of weights initialized using the given settings */
		UE_API TArrayView<float> MakeInitialWeights(
			const uint32 InputSize, 
			const uint32 OutputSize,
			const FWeightInitializationSettings& WeightInitializationSettings = FWeightInitializationSettings());

		/** Creates a array of biases initialized using the given settings */
		UE_API TArrayView<float> MakeInitialBiases(
			const uint32 OutputSize,
			const FWeightInitializationSettings& WeightInitializationSettings = FWeightInitializationSettings());

		/** Creates a array of weights randomly initialized using the given settings and compresses them */
		UE_API void MakeInitialCompressedWeights(
			TArrayView<uint16>& OutWeightsView,
			TArrayView<float>& OutWeightOffsetsView,
			TArrayView<float>& OutWeightScalesView,
			const uint32 InputSize,
			const uint32 OutputSize,
			const FWeightInitializationSettings& WeightInitializationSettings = FWeightInitializationSettings());

		/** Creates an array of sizes, initialized to zero. */
		UE_API TArrayView<uint32> MakeSizesZero(const uint32 Size);

		/** Creates an array of sizes from an array of builder elements' input sizes. */
		UE_API TArrayView<uint32> MakeSizesLayerInputs(const TConstArrayView<FModelBuilderElement> Elements);

		/** Creates an array of sizes from an array of builder elements' output sizes. */
		UE_API TArrayView<uint32> MakeSizesLayerOutputs(const TConstArrayView<FModelBuilderElement> Elements);

	public:

		/** Reset the builder clearing all memory. */
		UE_API void Reset();

		/**
		 * Get the number of bytes this builder currently wants to write for the given element.
		 */
		UE_API uint64 GetWriteByteNum(const FModelBuilderElement& Element) const;

		/**
		 * Write the Model to FileData. Use `GetWriteByteNum` to get the number of bytes this will write so that 
		 * `FileData` can be allocated to the right size.
		 * 
		 * @param OutFileData			Output File data to write to
		 * @param OutInputSize			Output for the number of float inputs to the model
		 * @param OutOutputSize			Output for the number of float outputs from the model
		 * @param Element				Builder element to write the model FileData for.
		 */
		UE_API void WriteFileData(TArrayView<uint8> OutFileData, uint32& OutInputSize, uint32& OutOutputSize, const FModelBuilderElement& Element) const;

		/**
		 * Write the model to the FileData. Use `GetWriteByteNum` to get the number of bytes this will write so that 
		 * `FileData` can be allocated to the right size.
		 *
		 * @param OutFileData			Output File data to write to
		 * @param OutInputSize			Output for the number of float inputs to the model
		 * @param OutOutputSize			Output for the number of float outputs from the model
		 * @param Element				Builder element to write the model FileData for.
		 */
		UE_API void WriteFileData(TArray<uint8>& OutFileData, uint32& OutInputSize, uint32& OutOutputSize, const FModelBuilderElement& Element) const;

		/**
		 * Write the model to the FileData and Reset the builder clearing all memory used.
		 *
		 * @param OutFileData			Output File data to write to
		 * @param OutInputSize			Output for the number of float inputs to the model
		 * @param OutOutputSize			Output for the number of float outputs from the model
		 * @param Element				Builder element to write the model FileData for.
		 */
		UE_API void WriteFileDataAndReset(TArrayView<uint8> OutFileData, uint32& OutInputSize, uint32& OutOutputSize, const FModelBuilderElement& Element);

		/**
		 * Write the model to the FileData and Reset the builder clearing all memory used.
		 * 
		 * @param OutFileData			Output File data to write to
		 * @param OutInputSize			Output for the number of float inputs to the model
		 * @param OutOutputSize			Output for the number of float outputs from the model
		 * @param Element				Builder element to write the model FileData for.
		 */
		UE_API void WriteFileDataAndReset(TArray<uint8>& OutFileData, uint32& OutInputSize, uint32& OutOutputSize, const FModelBuilderElement& Element);

	private:

		/** Default initial state for the random number generator. */
		static constexpr uint32 RngInitialState = 0xafcc2b45;

		/** Random Number State for generating random weights */
		uint32 Rng = RngInitialState;

		/** Pool of all weights data used by the `MakeValues`, `MakeWeights` `MakeBiases` functions. */
		TArray<TArray<float>> WeightsPool;

		/** Pool of all compressed weights data used by the `MakeCompressedWeights` functions. */
		TArray<TArray<uint16>> CompressedWeightsPool;

		/** Pool of all sizes data used by the `MakeSizes` functions. */
		TArray<TArray<uint32>> SizesPool;
	};

}

#undef UE_API 